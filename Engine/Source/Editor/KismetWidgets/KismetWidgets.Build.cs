// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class KismetWidgets : ModuleRules
{
	public KismetWidgets(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core", 
				"CoreUObject",
                "InputCore",
				"Engine",
				"Slate",
				"SlateCore",
				"EditorFramework",
				"UnrealEd",
				"BlueprintGraph",
				"ContentBrowserData",
				"ClassViewer",
				"ToolWidgets",
				"EditorWidgets",
			}
		);

        PrivateIncludePathModuleNames.AddRange(
            new string[] {
				"ContentBrowser",
                "AssetTools",
			}
		);

        DynamicallyLoadedModuleNames.AddRange(
            new string[] {
			    "ContentBrowser",
                "AssetTools",
			}
		);

		// Circular references that need to be cleaned up
		CircularlyReferencedDependentModules.AddRange(
			new string[] {
				"BlueprintGraph",
			}
		);
	}
}
