// Copyright Epic Games, Inc. All Rights Reserved.

#include "CreateBlueprintFromActorDialog.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetToolsModule.h"
#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "Components/ActorComponent.h"
#include "Containers/Array.h"
#include "Containers/Set.h"
#include "ContentBrowserDataSubsystem.h"
#include "ContentBrowserDelegates.h"
#include "ContentBrowserItemPath.h"
#include "ContentBrowserModule.h"
#include "Delegates/Delegate.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/Blueprint.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Notifications/NotificationManager.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMisc.h"
#include "IAssetTools.h"
#include "IContentBrowserSingleton.h"
#include "Input/Events.h"
#include "Input/Reply.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/FileHelper.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "PackageTools.h"
#include "SClassViewer.h"
#include "SPrimaryButton.h"
#include "SSimpleButton.h"
#include "Selection.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

struct FGeometry;

#define LOCTEXT_NAMESPACE "CreateBlueprintFromActorDialog"

class SSCreateBlueprintPicker : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSCreateBlueprintPicker)
	{}

	SLATE_ARGUMENT(TSharedPtr<SWindow>, ParentWindow)
		SLATE_ARGUMENT(AActor*, ActorOverride)
		SLATE_ARGUMENT(ECreateBlueprintFromActorMode, CreateMode)
		SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);

	/** Handler for when a class is picked in the class picker */
	void OnClassPicked(UClass* InChosenClass);

	/** Handler for when "Ok" we selected in the class viewer */
	FReply OnClassPickerConfirmed();

	/** Handler for when "Cancel" we selected in the class viewer */
	FReply OnClassPickerCanceled();

	/** Handler for when "..." is clicked to pick an asset path */
	FReply OnPathPickerSummoned();

	/** Handler for the custom button to hide/unhide the class viewer */
	void OnCustomAreaExpansionChanged(bool bExpanded);

	/** Callback when the user changes the filename for the Blueprint */
	void OnFilenameChanged(const FText& InNewName);

	/** Common function to evaluate whether the asset name given the asset path context, is valid */
	void UpdateFilenameStatus();

	/** select button visibility delegate */
	EVisibility GetSelectButtonVisibility() const;

	ECheckBoxState IsCreateModeChecked(ECreateBlueprintFromActorMode InCreateMode) const
	{
		return (CreateMode == InCreateMode ? ECheckBoxState::Checked : ECheckBoxState::Unchecked);
	};

	void OnCreateModeChanged(ECheckBoxState NewCheckedState, ECreateBlueprintFromActorMode InCreateMode) 
	{ 
		if (NewCheckedState == ECheckBoxState::Checked) 
		{ 
			CreateMode = InCreateMode; 
			ClassViewer->Refresh();
		} 	
	};

	FText GetCreateMethodTooltip(ECreateBlueprintFromActorMode InCreateMode, bool bEnabled) const;

	FSlateColor GetCreateModeTextColor(ECreateBlueprintFromActorMode InCreateMode) const
	{ 
		return (CreateMode == InCreateMode ? FSlateColor(FLinearColor(0, 0, 0)) : FSlateColor(FLinearColor(0.72f, 0.72f, 0.72f, 1.f))); 
	};

	/** Overridden from SWidget: Called when a key is pressed down - capturing copy */
	FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);

	/** A pointer to the window that is asking the user to select a parent class */
	TWeakPtr<SWindow> WeakParentWindow;

	/** A pointer to a class viewer **/
	TSharedPtr<SClassViewer> ClassViewer;

	/** Filename textbox widget */
	TSharedPtr<SEditableTextBox> FileNameWidget;

	/** The class that was last clicked on */
	UClass* ChosenClass;

	/** The actor that was passed in */
	TWeakObjectPtr<AActor> ActorOverride;

	/** The path the asset should be created at */
	FContentBrowserItemPath AssetPath;

	/** The the name for the new asset */
	FString AssetName;

	/** The method to use when creating the actor */
	ECreateBlueprintFromActorMode CreateMode;

	/** A flag indicating that Ok was selected */
	bool bPressedOk;

	/** A flag indicating the current asset name is invalid */
	bool bIsReportingError;
};

ECreateBlueprintFromActorMode FCreateBlueprintFromActorDialog::GetValidCreationMethods()
{
	int32 NumSelectedActors = 0;

	bool bCanHarvestComponents = false;
	bool bCanSubclass = true;
	bool bCanCreatePrefab = true;

	for (FSelectionIterator Iter(*GEditor->GetSelectedActors()); Iter; ++Iter)
	{
		AActor* Actor = Cast<AActor>(*Iter);
		if (Actor)
		{
			if (NumSelectedActors == 0)
			{
				bCanSubclass = FKismetEditorUtilities::CanCreateBlueprintOfClass(Actor->GetClass());
				if (bCanSubclass)
				{
					// Check whether the class is allowed by the global class filter
					FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");
					if (const TSharedPtr<IClassViewerFilter>& GlobalClassFilter = ClassViewerModule.GetGlobalClassViewerFilter())
					{
						TSharedRef<FClassViewerFilterFuncs> ClassFilterFuncs = ClassViewerModule.CreateFilterFuncs();
						FClassViewerInitializationOptions ClassViewerOptions = {};
						bCanSubclass = GlobalClassFilter->IsClassAllowed(ClassViewerOptions, Actor->GetClass(), ClassFilterFuncs);
					}
				}
			}

			if (bCanCreatePrefab && Actor->GetClass()->HasAnyClassFlags(CLASS_NotPlaceable))
			{
				bCanCreatePrefab = false;
			}

			if (!bCanHarvestComponents)
			{
				for (UActorComponent* Component : Actor->GetComponents())
				{
					if (Component && FKismetEditorUtilities::IsClassABlueprintSpawnableComponent(Component->GetClass()))
					{
						bCanHarvestComponents = true;
						break;
					}
				}
			}
		}
		++NumSelectedActors;
	}

	ECreateBlueprintFromActorMode ValidCreationMethods = ECreateBlueprintFromActorMode::None;
	if (NumSelectedActors > 0)
	{
		if (bCanHarvestComponents)
		{ 
			ValidCreationMethods |= ECreateBlueprintFromActorMode::Harvest;
		}
		if (bCanCreatePrefab)
		{
			ValidCreationMethods |= ECreateBlueprintFromActorMode::ChildActor;
		}
		if (bCanSubclass && NumSelectedActors == 1)
		{
			ValidCreationMethods |= ECreateBlueprintFromActorMode::Subclass;
		}
	}

	return ValidCreationMethods;
}

FText SSCreateBlueprintPicker::GetCreateMethodTooltip(ECreateBlueprintFromActorMode InCreateMode, bool bEnabled) const
{
	if (!bEnabled)
	{
		switch (InCreateMode)
		{
		case ECreateBlueprintFromActorMode::Subclass:
		{
			int32 NumSelectedActors = 0;
			UClass* SelectedActorClass = nullptr;
			for (FSelectionIterator Iter(*GEditor->GetSelectedActors()); Iter; ++Iter)
			{
				if (AActor* Actor = Cast<AActor>(*Iter))
				{
					SelectedActorClass = Actor->GetClass();
				}
				++NumSelectedActors;
			}

			if (NumSelectedActors == 1)
			{
				return FText::Format(LOCTEXT("SubClassDisabled_InvalidBlueprintType", "Cannot create blueprint subclass of '{0}'."), (SelectedActorClass ? SelectedActorClass->GetDisplayNameText() : FText::GetEmpty()));
			}
			else
			{
				return LOCTEXT("SubClassDisabled_MultipleSelection", "Cannot subclass when multiple actors are selected.");
			}
		}

		case ECreateBlueprintFromActorMode::ChildActor:
			return LOCTEXT("ChildActorDisabled", "No selected actor can be spawned as a child actor.");

		case ECreateBlueprintFromActorMode::Harvest:
			return LOCTEXT("HavestDisabled", "No harvestable components in selected actors.");

		}
	}

	return FText::GetEmpty();
}

void SSCreateBlueprintPicker::Construct(const FArguments& InArgs)
{
	WeakParentWindow = InArgs._ParentWindow;

	bPressedOk = false;
	ChosenClass = nullptr;

	ECreateBlueprintFromActorMode ValidCreateMethods = FCreateBlueprintFromActorDialog::GetValidCreationMethods();
	const bool bCanHarvestComponents = !!(ValidCreateMethods & ECreateBlueprintFromActorMode::Harvest);
	const bool bCanSubclass = !!(ValidCreateMethods & ECreateBlueprintFromActorMode::Subclass);
	const bool bCanCreatePrefab = !!(ValidCreateMethods & ECreateBlueprintFromActorMode::ChildActor);

	if (!!(InArgs._CreateMode & ValidCreateMethods))
	{
		CreateMode = InArgs._CreateMode;
	}
	else
	{
		if (bCanSubclass)
		{
			CreateMode = ECreateBlueprintFromActorMode::Subclass;
		}
		else if (bCanCreatePrefab)
		{
			CreateMode = ECreateBlueprintFromActorMode::ChildActor;
		}
		else if (bCanHarvestComponents)
		{
			CreateMode = ECreateBlueprintFromActorMode::Harvest;
		}
		else
		{
			CreateMode = ECreateBlueprintFromActorMode::None;
		}
	}

	// Set initial destination asset folder and name
	{
		FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

		AssetPath = ContentBrowserModule.Get().GetCurrentPath();
		// Change path if cannot write to it
		AssetPath = ContentBrowserModule.Get().GetInitialPathToSaveAsset(AssetPath);

		for (FSelectionIterator Iter(*GEditor->GetSelectedActors()); Iter; ++Iter)
		{
			AActor* Actor = Cast<AActor>(*Iter);
			if (Actor)
			{
				AssetName += Actor->GetActorLabel();
				AssetName += TEXT("_");
				break;
			}
		}

		AssetName = UPackageTools::SanitizePackageName(AssetName + TEXT("Blueprint"));
	}

	ActorOverride = InArgs._ActorOverride;

	FClassViewerInitializationOptions ClassViewerOptions;
	ClassViewerOptions.Mode = EClassViewerMode::ClassPicker;
	ClassViewerOptions.DisplayMode = EClassViewerDisplayMode::TreeView;
	ClassViewerOptions.bShowObjectRootClass = true;
	ClassViewerOptions.bIsPlaceableOnly = true;
	ClassViewerOptions.bIsBlueprintBaseOnly = true;
	ClassViewerOptions.bShowUnloadedBlueprints = true;
	ClassViewerOptions.bEnableClassDynamicLoading = true;
	ClassViewerOptions.NameTypeToDisplay = EClassViewerNameTypeToDisplay::Dynamic;

	if (ActorOverride == nullptr)
	{
		USelection* SelectedActors = GEditor->GetSelectedActors();
		if (SelectedActors->Num() == 1)
		{
			ActorOverride = CastChecked<AActor>(SelectedActors->GetSelectedObject(0));
		}
	}

	class FBlueprintFromActorParentFilter : public IClassViewerFilter
	{
	public:
		FBlueprintFromActorParentFilter(UClass* InAllowedClass, ECreateBlueprintFromActorMode& InCreateModeRef)
			: CreateModeRef(InCreateModeRef)
		{
			AllowedClass.Add(InAllowedClass);
		}

		TSet<const UClass*> AllowedClass;
		const ECreateBlueprintFromActorMode& CreateModeRef;

		virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
		{
			return CreateModeRef != ECreateBlueprintFromActorMode::Subclass || InFilterFuncs->IfInChildOfClassesSet(AllowedClass, InClass) == EFilterReturn::Passed;
		}

		virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
		{
			return CreateModeRef != ECreateBlueprintFromActorMode::Subclass || InFilterFuncs->IfInChildOfClassesSet(AllowedClass, InUnloadedClassData) == EFilterReturn::Passed;
		}
	};

	UClass* ActorOverrideClass = nullptr;
	if (ActorOverride.IsValid())
	{
		ActorOverrideClass = ActorOverride->GetClass();
		TSharedPtr<FBlueprintFromActorParentFilter> Filter = MakeShareable(new FBlueprintFromActorParentFilter(ActorOverrideClass, CreateMode));
		ClassViewerOptions.ClassFilters.Add(Filter.ToSharedRef());
	}

	if (ActorOverrideClass && CreateMode == ECreateBlueprintFromActorMode::Subclass)
	{
		ClassViewerOptions.InitiallySelectedClass = ActorOverrideClass;
	}
	else
	{
		ClassViewerOptions.InitiallySelectedClass = AActor::StaticClass();
	}

	{
		const FString DestPackageName = FPaths::Combine(AssetPath.GetInternalPathString(), AssetName);
		const FString DestAssetPath = FString::Printf(TEXT("%s.%s"), *DestPackageName, *AssetName);
		ClassViewerOptions.AdditionalReferencingAssets.Add(FAssetData(DestPackageName, DestAssetPath, UBlueprint::StaticClass()->GetClassPathName()));
		// @fixme: Update the class viewer whenever the destination folder changes
	}

	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");
	ClassViewer = StaticCastSharedRef<SClassViewer>(ClassViewerModule.CreateClassViewer(ClassViewerOptions, FOnClassPicked::CreateSP(this, &SSCreateBlueprintPicker::OnClassPicked)));

	FString PackageName;
	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	AssetToolsModule.Get().CreateUniqueAssetName(AssetPath.GetInternalPathString() / AssetName, TEXT(""), PackageName, AssetName);

	TSharedPtr<SGridPanel> CreationMethodSection;

	struct FCreateModeDetails
	{
		FText Label;
		FText Description;
		ECreateBlueprintFromActorMode CreateMode;
		bool bEnabled;
	};

	const FCreateModeDetails CreateModeDetails[3] =
	{
		{ LOCTEXT("CreateMode_Subclass", "New Subclass"), LOCTEXT("CreateMode_Subclass_Description", "Replace the selected actor with an instance of a new Blueprint Class inherited from the selected parent class."), ECreateBlueprintFromActorMode::Subclass, bCanSubclass },
		{ LOCTEXT("CreateMode_ChildActor", "Child Actors"), LOCTEXT("CreateMode_ChildActor_Description", "Replace the selected actors with an instance of a new Blueprint Class inherited from the selected parent class with each of the selected Actors as a Child Actor."), ECreateBlueprintFromActorMode::ChildActor, bCanCreatePrefab },
		{ LOCTEXT("CreateMode_Harvest", "Harvest Components"), LOCTEXT("CreateMode_Harvest_Description", "Replace the selected actors with an instance of a new Blueprint Class inherited from the selected parent class that contains the components."), ECreateBlueprintFromActorMode::Harvest, bCanHarvestComponents }
	};

	const FCheckBoxStyle& RadioStyle = FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("SegmentedCombo.ButtonOnly");

	SAssignNew(CreationMethodSection, SGridPanel)
	.FillColumn(1, 1.f);

	for (int32 Index = 0; Index < UE_ARRAY_COUNT(CreateModeDetails); ++Index)
	{
		float TopPadding = Index == 0 ? 28.0f : 16.0f;
		float BottomPadding = Index == UE_ARRAY_COUNT(CreateModeDetails) - 1 ? 28.0f : 16.0f;

		CreationMethodSection->AddSlot(0, Index)
		.Padding(10.0f, TopPadding, 5.0f, BottomPadding)
		.VAlign(VAlign_Center)
		[
			SNew(SCheckBox)
			.Style(&RadioStyle)
			.IsEnabled(CreateModeDetails[Index].bEnabled)
			.IsChecked_Raw(this, &SSCreateBlueprintPicker::IsCreateModeChecked, CreateModeDetails[Index].CreateMode)
			.OnCheckStateChanged(this, &SSCreateBlueprintPicker::OnCreateModeChanged, CreateModeDetails[Index].CreateMode)
			.ToolTipText_Raw(this, &SSCreateBlueprintPicker::GetCreateMethodTooltip, CreateModeDetails[Index].CreateMode, CreateModeDetails[Index].bEnabled)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				.Padding(-8, 3, 0, 3)
				.AutoWidth()
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Blueprints"))
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				.Padding(4, 3, 0, 3)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(CreateModeDetails[Index].Label)
					.TextStyle(FAppStyle::Get(), "NormalText")
					.ColorAndOpacity(FStyleColors::White)
				]
			]
		];

		CreationMethodSection->AddSlot(1, Index)
		.Padding(4.0f, TopPadding, 1.0f, BottomPadding)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(CreateModeDetails[Index].Description)
			.TextStyle(FAppStyle::Get(), "SmallText")
			.IsEnabled(CreateModeDetails[Index].bEnabled)
			.AutoWrapText(true)
		];

	}

	ChildSlot
	[
		SNew(SBorder)
		.Visibility(EVisibility::Visible)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
		[
			SNew(SBox)
			.Visibility(EVisibility::Visible)
			[
				SNew(SVerticalBox)

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(SExpandableArea)
					.AreaTitle(LOCTEXT("CreationMethod", "Creation Method"))
					.AreaTitleFont(FCoreStyle::Get().GetFontStyle("NormalFontBold"))
					.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
					.BodyBorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
					.BodyContent()
					[
						CreationMethodSection.ToSharedRef()
					]
				]
				
				+SVerticalBox::Slot()
				.FillHeight(1.f)
				.Padding(0.0f, 1.0f, 0.0f, 0.0f)
				[
					SNew(SExpandableArea)
					.MaxHeight(320.f)
					.InitiallyCollapsed(false)
					.AreaTitle(NSLOCTEXT("SClassPickerDialog", "ParentClassAreaTitle", "Parent Class"))
					.AreaTitleFont(FCoreStyle::Get().GetFontStyle("NormalFontBold"))
					.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
					.BodyBorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
					.OnAreaExpansionChanged(this, &SSCreateBlueprintPicker::OnCustomAreaExpansionChanged)
					.BodyContent()
					[
						ClassViewer.ToSharedRef()
					]
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 10.0f, 0.0f, 0.0f)
				[
					SNew(SGridPanel)
					.FillColumn(1, 1.f)
					+SGridPanel::Slot(0, 0)
					.Padding(16.0f, 0.0f, 13.0f, 7.0f)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("CreateBlueprintFromActor_NameLabel", "Blueprint Name"))
					]
					+SGridPanel::Slot(1, 0)
					.Padding(0.0f, 0.0f, 55.0f, 10.0f)
					[
						SAssignNew(FileNameWidget, SEditableTextBox)
						.Text(FText::FromString(AssetName))
						.OnTextChanged(this, &SSCreateBlueprintPicker::OnFilenameChanged)
					]
					+SGridPanel::Slot(0, 1)
					.Padding(15.0f, 0.0f, 13.0f, 0.0f)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("CreateBlueprintFromActor_PathLabel", "Path"))
					]
					+SGridPanel::Slot(1, 1)
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.FillWidth(1.f)
						[
							SNew(SEditableTextBox)
							.Text(TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateLambda([this]() { return FText::FromName(AssetPath.GetVirtualPathName()); })))
							.IsReadOnly(true)
						]
						+SHorizontalBox::Slot()
						.Padding(8.0f, 0.0f, 19.0f, 0.0f)
						.AutoWidth()
						[
							SNew(SSimpleButton)
							.OnClicked(this, &SSCreateBlueprintPicker::OnPathPickerSummoned)
							.Icon(FAppStyle::Get().GetBrush("Icons.FolderClosed"))
						]
					]
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Bottom)
				.Padding(8.0f)
				[
					SNew(SUniformGridPanel)
					.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
					+SUniformGridPanel::Slot(0,0)
					[
						SNew(SPrimaryButton)
						.Text(NSLOCTEXT("SClassPickerDialog", "ClassPickerSelectButton", "Select"))
						.Visibility( this, &SSCreateBlueprintPicker::GetSelectButtonVisibility )
						.OnClicked(this, &SSCreateBlueprintPicker::OnClassPickerConfirmed)
					]
					+SUniformGridPanel::Slot(1,0)
					[
						SNew(SButton)
						.Text(NSLOCTEXT("SClassPickerDialog", "ClassPickerCancelButton", "Cancel"))
						.OnClicked(this, &SSCreateBlueprintPicker::OnClassPickerCanceled)

					]
				]
			]
		]
	];

	if (WeakParentWindow.IsValid())
	{
		WeakParentWindow.Pin().Get()->SetWidgetToFocusOnActivate(ClassViewer);
	}
}

void SSCreateBlueprintPicker::OnClassPicked(UClass* InChosenClass)
{
	ChosenClass = InChosenClass;
}

FReply SSCreateBlueprintPicker::OnClassPickerConfirmed()
{
	if (ChosenClass == nullptr)
	{
		FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("EditorFactories", "MustChooseClassWarning", "You must choose a class."));
	}
	else if (bIsReportingError)
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("InvalidAssetname", "You must specify a valid asset name."));
	}
	else
	{
		bPressedOk = true;

		if (WeakParentWindow.IsValid())
		{
			WeakParentWindow.Pin()->RequestDestroyWindow();
		}
	}
	return FReply::Handled();
}

FReply SSCreateBlueprintPicker::OnClassPickerCanceled()
{
	if (WeakParentWindow.IsValid())
	{
		WeakParentWindow.Pin()->RequestDestroyWindow();
	}
	return FReply::Handled();
}

class SSCreateBlueprintPathPicker : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SSCreateBlueprintPathPicker)
	{}

	SLATE_ARGUMENT(TSharedPtr<SWindow>, ParentWindow)
		SLATE_ARGUMENT(FContentBrowserItemPath, AssetPath)
		SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/** Callback when the selected asset path has changed. */
	void OnSelectAssetPath(const FString& InVirtualPath) { AssetPath.SetPathFromString(InVirtualPath, EContentBrowserPathType::Virtual); }

	/** Callback when the "ok" button is clicked. */
	FReply OnClickOk();

	/** Destroys the window when the operation is cancelled. */
	FReply OnClickCancel();

	/** A pointer to the window that is asking the user to select a parent class */
	TWeakPtr<SWindow> WeakParentWindow;

	FContentBrowserItemPath AssetPath;

	bool bPressedOk;
};

void SSCreateBlueprintPathPicker::Construct(const FArguments& InArgs)
{
	WeakParentWindow = InArgs._ParentWindow;

	bPressedOk = false;
	AssetPath = InArgs._AssetPath;

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	FPathPickerConfig PathPickerConfig;
	PathPickerConfig.DefaultPath = AssetPath.GetVirtualPathString();
	PathPickerConfig.OnPathSelected = FOnPathSelected::CreateRaw(this, &SSCreateBlueprintPathPicker::OnSelectAssetPath);
	PathPickerConfig.bAllowReadOnlyFolders = false;
	PathPickerConfig.bOnPathSelectedPassesVirtualPaths = true;

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			ContentBrowserModule.Get().CreatePathPicker(PathPickerConfig)
		]
		+ SVerticalBox::Slot()
		.HAlign(HAlign_Right)
		.Padding(0, 20, 0, 0)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(0, 2, 6, 0)
			.AutoWidth()
			[
				SNew(SButton)
				.VAlign(VAlign_Bottom)
				.ContentPadding(FMargin(8, 2, 8, 2))
				.OnClicked(this, &SSCreateBlueprintPathPicker::OnClickOk)
				.ButtonStyle(FAppStyle::Get(), "FlatButton.Success")
				.TextStyle(FAppStyle::Get(), "FlatButton.DefaultTextStyle")
				.Text(LOCTEXT("OkButtonText", "OK"))
			]
			+ SHorizontalBox::Slot()
			.Padding(0, 2, 0, 0)
			.AutoWidth()
			[
				SNew(SButton)
				.VAlign(VAlign_Bottom)
				.ContentPadding(FMargin(8, 2, 8, 2))
				.OnClicked(this, &SSCreateBlueprintPathPicker::OnClickCancel)
				.ButtonStyle(FAppStyle::Get(), "FlatButton.Default")
				.TextStyle(FAppStyle::Get(), "FlatButton.DefaultTextStyle")
				.Text(LOCTEXT("CancelButtonText", "Cancel"))
			]
		]
	];
};

FReply SSCreateBlueprintPathPicker::OnClickOk()
{
	bPressedOk = true;

	if (WeakParentWindow.IsValid())
	{
		WeakParentWindow.Pin()->RequestDestroyWindow();
	}
	return FReply::Handled();
}

FReply SSCreateBlueprintPathPicker::OnClickCancel()
{
	if (WeakParentWindow.IsValid())
	{
		WeakParentWindow.Pin()->RequestDestroyWindow();
	}
	return FReply::Handled();
}

FReply SSCreateBlueprintPicker::OnPathPickerSummoned()
{
	// Create the window to pick the class
	TSharedRef<SWindow> PickerWindow = SNew(SWindow)
		.Title(LOCTEXT("CreateBlueprintFromActors_PickPath", "Select Path"))
		.SizingRule(ESizingRule::UserSized)
		.ClientSize(FVector2D(300.f, 400.f))
		.SupportsMaximize(false)
		.SupportsMinimize(false);

	TSharedRef<SSCreateBlueprintPathPicker> PathPickerDialog = SNew(SSCreateBlueprintPathPicker)
		.ParentWindow(PickerWindow)
		.AssetPath(AssetPath);

	PickerWindow->SetContent(PathPickerDialog);

	GEditor->EditorAddModalWindow(PickerWindow);

	if (PathPickerDialog->bPressedOk)
	{
		AssetPath.SetPathFromString(PathPickerDialog->AssetPath.GetVirtualPathString(), EContentBrowserPathType::Virtual);
		UpdateFilenameStatus();
	}

	return FReply::Handled();
}

void SSCreateBlueprintPicker::OnCustomAreaExpansionChanged(bool bExpanded)
{
	if (bExpanded && WeakParentWindow.IsValid())
	{
		WeakParentWindow.Pin().Get()->SetWidgetToFocusOnActivate(ClassViewer);
	}
}

void SSCreateBlueprintPicker::OnFilenameChanged(const FText& InNewName)
{
	AssetName = InNewName.ToString();
	UpdateFilenameStatus();
}

void SSCreateBlueprintPicker::UpdateFilenameStatus()
{
	FText ErrorText;
	if (!FFileHelper::IsFilenameValidForSaving(AssetName, ErrorText) || !FName(*AssetName).IsValidObjectName(ErrorText))
	{
		FileNameWidget->SetError(ErrorText);
		bIsReportingError = true;
		return;
	}
	else
	{
		TArray<FAssetData> AssetData;
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		AssetRegistryModule.Get().GetAssetsByPath(AssetPath.GetInternalPathName(), AssetData);

		// Check to see if the name conflicts
		for (const FAssetData& Data : AssetData)
		{
			const FString& AssetNameStr = Data.AssetName.ToString();
			if (Data.AssetName.ToString() == AssetName)
			{
				FileNameWidget->SetError(LOCTEXT("AssetInUseError", "Asset name already in use!"));
				bIsReportingError = true;
				return;
			}
		}
	}

	FileNameWidget->SetError(FText::GetEmpty());
	bIsReportingError = false;
}

EVisibility SSCreateBlueprintPicker::GetSelectButtonVisibility() const
{
	EVisibility ButtonVisibility = EVisibility::Visible;
	if (ChosenClass == nullptr || bIsReportingError)
	{
		ButtonVisibility = EVisibility::Hidden;
	}
	else if (CreateMode == ECreateBlueprintFromActorMode::Subclass)
	{
		UObject* SelectedActor = GEditor->GetSelectedActors()->GetSelectedObject(0);

		if (SelectedActor == nullptr || !ChosenClass->IsChildOf(SelectedActor->GetClass()))
		{
			ButtonVisibility = EVisibility::Hidden;
		}
	}
	return ButtonVisibility;
}

/** Overridden from SWidget: Called when a key is pressed down - capturing copy */
FReply SSCreateBlueprintPicker::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	WeakParentWindow.Pin().Get()->SetWidgetToFocusOnActivate(ClassViewer);

	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		return OnClassPickerCanceled();
	}
	else if (InKeyEvent.GetKey() == EKeys::Enter)
	{
		OnClassPickerConfirmed();
	}
	else
	{
		return ClassViewer->OnKeyDown(MyGeometry, InKeyEvent);
	}
	return FReply::Handled();
}

void FCreateBlueprintFromActorDialog::OpenDialog(ECreateBlueprintFromActorMode CreateMode, AActor* InActorOverride, bool bInReplaceActors)
{
	TWeakObjectPtr<AActor> ActorOverride(InActorOverride);

	// Create the window to pick the class
	TSharedRef<SWindow> PickerWindow = SNew(SWindow)
		.Title(LOCTEXT("CreateBlueprintFromActors","Create Blueprint From Selection"))
		.SizingRule(ESizingRule::UserSized)
		.ClientSize(FVector2D(600.f, 600.f))
		.SupportsMaximize(false)
		.SupportsMinimize(false);

	TSharedRef<SSCreateBlueprintPicker> ClassPickerDialog = SNew(SSCreateBlueprintPicker)
		.ParentWindow(PickerWindow)
		.ActorOverride(InActorOverride)
		.CreateMode(CreateMode);

	PickerWindow->SetContent(ClassPickerDialog);
		
	GEditor->EditorAddModalWindow(PickerWindow);

	if (ClassPickerDialog->bPressedOk)
	{
		if (ClassPickerDialog->AssetPath.HasInternalPath())
		{
			FString NewAssetName = ClassPickerDialog->AssetPath.GetInternalPathString() / ClassPickerDialog->AssetName;
			OnCreateBlueprint(NewAssetName, ClassPickerDialog->ChosenClass, ClassPickerDialog->CreateMode, ActorOverride.Get(), bInReplaceActors);
		}
		else
		{
			FNotificationInfo ErrorNotificationInfo(FText::FormatOrdered(LOCTEXT("PathError", "Could not convert virtual path '{0}' to internal path."), FText::FromString(*ClassPickerDialog->AssetPath.GetVirtualPathString())));
			TSharedPtr<SNotificationItem> NotificationItem = FSlateNotificationManager::Get().AddNotification(ErrorNotificationInfo);
			NotificationItem->SetCompletionState(SNotificationItem::CS_Fail);
		}
	}
}

void FCreateBlueprintFromActorDialog::OnCreateBlueprint(const FString& InAssetPath, UClass* ParentClass, ECreateBlueprintFromActorMode CreateMode, AActor* ActorToUse, bool bInReplaceActors)
{
	UBlueprint* Blueprint = nullptr;

	switch (CreateMode) 
	{
		case ECreateBlueprintFromActorMode::Harvest:
		{
			TArray<AActor*> Actors;

			USelection* SelectedActors = GEditor->GetSelectedActors();
			for(FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
			{
				// We only care about actors that are referenced in the world for literals, and also in the same level as this blueprint
					if (AActor* Actor = Cast<AActor>(*Iter))
				{
					Actors.Add(Actor);
				}
			}

			FKismetEditorUtilities::FHarvestBlueprintFromActorsParams Params;
			Params.bReplaceActors = true;
			Params.ParentClass = ParentClass;
			Blueprint = FKismetEditorUtilities::HarvestBlueprintFromActors(InAssetPath, Actors, Params);
		}
		break;

		case ECreateBlueprintFromActorMode::Subclass:
		{
			if (!ActorToUse)
			{
				TArray<UObject*> SelectedActors;
				GEditor->GetSelectedActors()->GetSelectedObjects(AActor::StaticClass(), SelectedActors);
				check(SelectedActors.Num() == 1);
				ActorToUse = Cast<AActor>(SelectedActors[0]);
			}

			FKismetEditorUtilities::FCreateBlueprintFromActorParams Params;
			Params.bReplaceActor = true;
			Params.ParentClassOverride = ParentClass;
			Blueprint = FKismetEditorUtilities::CreateBlueprintFromActor(InAssetPath, ActorToUse, Params);
		}
		break;

		case ECreateBlueprintFromActorMode::ChildActor:
		{
			TArray<AActor*> Actors;

			USelection* SelectedActors = GEditor->GetSelectedActors();
			for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
			{
				// We only care about actors that are referenced in the world for literals, and also in the same level as this blueprint
				if (AActor* Actor = Cast<AActor>(*Iter))
				{
					Actors.Add(Actor);
				}
			}

			FKismetEditorUtilities::FCreateBlueprintFromActorsParams Params(Actors);
			Params.bReplaceActors = true;
			Params.ParentClass = ParentClass;

			Blueprint = FKismetEditorUtilities::CreateBlueprintFromActors(InAssetPath, Params);
		}
		break;
	}

	if (Blueprint)
	{
		// Select the newly created blueprint in the content browser, but don't activate the browser
		TArray<UObject*> Objects;
		Objects.Add(Blueprint);
		GEditor->SyncBrowserToObjects( Objects, false );
	}
	else
	{
		FNotificationInfo Info( LOCTEXT("CreateBlueprintFromActorFailed", "Unable to create a blueprint from actor.") );
		Info.ExpireDuration = 3.0f;
		Info.bUseLargeFont = false;
		TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
		if ( Notification.IsValid() )
		{
			Notification->SetCompletionState( SNotificationItem::CS_Fail );
		}
	}
}


#undef LOCTEXT_NAMESPACE
