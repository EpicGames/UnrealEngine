// Copyright Epic Games, Inc. All Rights Reserved.

#include "SKismetLinearExpression.h"

#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "K2Node_VariableGet.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;

#define LOCTEXT_NAMESPACE "KismetLinearExpression"

//////////////////////////////////////////////////////////////////////////
// SKismetLinearExpression

void SKismetLinearExpression::Construct(const FArguments& InArgs, UEdGraphPin* InitialInputPin)
{
	SetExpressionRoot(InitialInputPin);
}

void SKismetLinearExpression::SetExpressionRoot(UEdGraphPin* InputPin)
{
	this->ChildSlot
	[
		MakePinWidget(InputPin)
	];
	VisitedNodes.Empty();
}

TSharedRef<SWidget> SKismetLinearExpression::MakeNodeWidget(const UEdGraphNode* Node, const UEdGraphPin* FromPin)
{
	// Make sure we visit each node only once; to prevent infinite recursion
	bool bAlreadyInSet = false;
	VisitedNodes.Add(Node, &bAlreadyInSet);
	if (bAlreadyInSet)
	{
		return SNew(STextBlock).Text(LOCTEXT("RecursionOccurredInNodeGraphMessage", "RECURSION"));
	}

	// Evaluate the node to gather information that many of the widgets will want
	const UEdGraphSchema* Schema = Node->GetSchema();
	TArray<UEdGraphPin*> InputPins;
	TArray<UEdGraphPin*> OutputPins;
	int32 InputPinCount = 0;
	int32 OutputPinCount = 0;
	for (auto PinIt = Node->Pins.CreateConstIterator(); PinIt; ++PinIt)
	{
		UEdGraphPin* Pin = *PinIt;
		if (!Pin->bHidden)
		{
			if (Pin->Direction == EGPD_Input)
			{
				InputPins.Add(Pin);
				++InputPinCount;
			}
			else
			{
				OutputPins.Add(Pin);
				++OutputPinCount;
			}
		}
	}

	// Determine if the node is impure
	bool bIsPure = false;
	if (const UK2Node* K2Node = Cast<const UK2Node>(Node))
	{
		bIsPure = K2Node->IsNodePure();
	}

	// If the node is 
	if ((OutputPinCount != 1) || !bIsPure)
	{
		// The source node is impure or has multiple outputs, so cannot be directly part of this pure expression
		// Instead show it as a special sort of variable get
		FFormatNamedArguments Args;
		Args.Add(TEXT("NodeTitle"), Node->GetNodeTitle(ENodeTitleType::ListView));
		Args.Add(TEXT("PinName"), FromPin->GetDisplayName());
		const FText EffectiveVariableName = FText::Format(LOCTEXT("NodeTitleWithPinName", "{NodeTitle}_{PinName}"), Args );

		return SNew(SOverlay)
			+ SOverlay::Slot()
			[
				SNew(SImage)
				.Image( FAppStyle::GetBrush("KismetExpression.ReadAutogeneratedVariable.Body") )
				.ColorAndOpacity( Schema->GetPinTypeColor(FromPin->PinType) )
			]
			+ SOverlay::Slot()
			.Padding( FMargin(6,4) )
			[
				SNew(STextBlock)
				.TextStyle( FAppStyle::Get(), TEXT("KismetExpression.ReadAutogeneratedVariable") )
				.Text( EffectiveVariableName )
			];
	}
	else if (auto VarGetNode = Cast<const UK2Node_VariableGet>(Node))
	{
		// Variable get node
		return SNew(SOverlay)
			+ SOverlay::Slot()
			[
				SNew(SImage)
				.Image( FAppStyle::GetBrush("KismetExpression.ReadVariable.Body") )
				.ColorAndOpacity( Schema->GetPinTypeColor(OutputPins[0]->PinType) )
			]
			+ SOverlay::Slot()
			.Padding( FMargin(6,4) )
			[
				//SNew(STextBlock)
				SNew(SEditableText)
				//.TextStyle( FAppStyle::Get(), TEXT("KismetExpression.ReadVariable") )
				.Text( FText::FromString( VarGetNode->GetVarNameString() ) )
			]
			+ SOverlay::Slot()
			[
				SNew(SImage)
				.Visibility(EVisibility::HitTestInvisible)
				.Image( FAppStyle::GetBrush("KismetExpression.ReadVariable.Gloss") )
			];
		/*
			+ SOverlay::Slot()
			.Padding( FMargin(6,4) )
			[
				SNew(STextBlock)
				.TextStyle( FAppStyle::Get(), TEXT("KismetExpression.ReadVariable") )
				.Text( VarGetNode->VariableName.ToString() )
			];
			*/
	}
	else if (const UK2Node* AnyNode = Cast<const UK2Node>(Node))
	{
		const bool bIsCompact = AnyNode->ShouldDrawCompact() && (InputPinCount <= 2);

		TSharedRef<SWidget> OperationWidget = 
			SNew(STextBlock)
			.TextStyle( FAppStyle::Get(), bIsCompact ? TEXT("KismetExpression.OperatorNode") : TEXT("KismetExpression.FunctionNode") )
			.Text(AnyNode->GetCompactNodeTitle());

		if ((InputPinCount == 1) && bIsCompact)
		{
			// One-pin compact nodes are assumed to be unary operators
			return SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4, 0)
				[
					OperationWidget
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					MakePinWidget(InputPins[0])
				];
		}
		else if ((InputPinCount == 2) && bIsCompact)
		{
			// Two-pin compact nodes are assumed to be binary operators
			return SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					MakePinWidget(InputPins[0])
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4, 0)
				[
					OperationWidget
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					MakePinWidget(InputPins[1])
				];
		}
		else
		{
			// All other operations are treated as traditional function calls


			// Create the argument list
			TSharedRef<SHorizontalBox> InnerBox =
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4.0f)
				[
					SNew(STextBlock).Text(LOCTEXT("BeginExpression", "("))
				];

			for (auto PinIt = InputPins.CreateConstIterator(); PinIt; )
			{
				InnerBox->AddSlot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					MakePinWidget(*PinIt)
				];

				++PinIt;

				if (PinIt)
				{
					InnerBox->AddSlot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(4)
					[
						SNew(STextBlock).Text(LOCTEXT("NextExpression", ", "))
					];
				}
			}

			InnerBox->AddSlot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4)
			[
				SNew(STextBlock).Text(LOCTEXT("EndExpression", ")"))
			];

			// Combine the function name and argument list
			return SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					OperationWidget
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					InnerBox
				];
		}
	}
	else
	{
		return SNew(STextBlock).Text(LOCTEXT("UnknownNodeMessage", "UNKNOWN_NODE"));
	}
}

TSharedRef<SWidget> SKismetLinearExpression::MakePinWidget(const UEdGraphPin* Pin)
{
	if (Pin == NULL)
	{
		return SNew(STextBlock)
			.Text( LOCTEXT("BanGraphPinMessage", "BAD PIN") );
	}
	else if (Pin->LinkedTo.Num() == 0)
	{
		// Input pins with no links are displayed as their literals
		return SNew(STextBlock)
			.TextStyle( FAppStyle::Get(), TEXT("KismetExpression.LiteralValue") )
			.Text(FText::FromString(Pin->GetDefaultAsString()));
	}
	else
	{
		// Evaluate the node they're connected to
		UEdGraphPin* SourcePin = Pin->LinkedTo[0];
		return MakeNodeWidget(SourcePin->GetOwningNode(), SourcePin);
	}
}



#undef LOCTEXT_NAMESPACE
