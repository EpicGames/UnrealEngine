// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLevelOfDetailBranchNode.h"

#include "CoreTypes.h"
#include "Layout/Children.h"
#include "Widgets/SNullWidget.h"

struct FGeometry;

/////////////////////////////////////////////////////
// SLevelOfDetailBranchNode

SLevelOfDetailBranchNode::SLevelOfDetailBranchNode()
	: LastCachedValue(INDEX_NONE)
	, ChildSlotLowDetail(SNullWidget::NullWidget)
	, ChildSlotHighDetail(SNullWidget::NullWidget)
{
}

void SLevelOfDetailBranchNode::Construct(const FArguments& InArgs)
{
	OnGetActiveDetailSlotContent = InArgs._OnGetActiveDetailSlotContent;
	ShowLowDetailAttr = InArgs._UseLowDetailSlot;
	ChildSlotLowDetail = InArgs._LowDetail.Widget;
	ChildSlotHighDetail = InArgs._HighDetail.Widget;

	RefreshLODSlotContent();
}

void SLevelOfDetailBranchNode::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	RefreshLODSlotContent();
}

void SLevelOfDetailBranchNode::RefreshLODSlotContent()
{
	const int32 CurrentValue = ShowLowDetailAttr.Get() ? 1 : 0;
	if (CurrentValue != LastCachedValue)
	{
		LastCachedValue = CurrentValue;

		if (OnGetActiveDetailSlotContent.IsBound())
		{
			ChildSlot
				[
					OnGetActiveDetailSlotContent.Execute((CurrentValue != 0) ? false : true)
				];
		}
		else
		{
			ChildSlot
				[
					(CurrentValue != 0) ? ChildSlotLowDetail : ChildSlotHighDetail
				];
		}
	}
}