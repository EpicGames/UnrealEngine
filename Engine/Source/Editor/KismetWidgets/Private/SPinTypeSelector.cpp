// Copyright Epic Games, Inc. All Rights Reserved.
#include "SPinTypeSelector.h"

#include "Algo/AnyOf.h"
#include "Algo/Count.h"
#include "Algo/LevenshteinDistance.h"
#include "Algo/RemoveIf.h"
#include "BlueprintEditorSettings.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraphSchema.h"
#include "Engine/Blueprint.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Framework/SlateDelegates.h"
#include "Framework/Views/ITypedTableView.h"
#include "IDocumentation.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Layout/BasicLayoutWidgetSlot.h"
#include "Layout/Children.h"
#include "Layout/Clipping.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "PinTypeSelectorFilter.h"
#include "SListViewSelectorDropdownMenu.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Styling/ISlateStyle.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SLayeredImage.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SSubMenuHandler.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SMenuOwner.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"

class ITableRow;
class SWidget;
struct FGeometry;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "PinTypeSelector"


/** Manages items in the Object Reference Type list, the sub-menu of the PinTypeSelector */
struct FObjectReferenceType
{
	/** Item that is being referenced */
	FPinTypeTreeItem PinTypeItem;

	/** Widget to display for this item */
	TSharedPtr< SWidget > WidgetToDisplay;

	/** Category that should be used when this item is selected */
	FName PinCategory;

	FObjectReferenceType(FPinTypeTreeItem InPinTypeItem, TSharedRef< SWidget > InWidget, FName InPinCategory)
		: PinTypeItem(InPinTypeItem)
		, WidgetToDisplay(InWidget)
		, PinCategory(InPinCategory)
	{}
};

namespace PinTypeSelectorStatics
{
	static const FString BigTooltipDocLink = TEXT("Shared/Editor/Blueprint/VariableTypes");

	// SComboBox is a bit restrictive:
	static TArray<TSharedPtr<EPinContainerType>> PinTypes;

	static FName Images[] = {
		TEXT("Kismet.VariableList.TypeIcon"),
		TEXT("Kismet.VariableList.ArrayTypeIcon"),
		TEXT("Kismet.VariableList.SetTypeIcon"),
		TEXT("Kismet.VariableList.MapKeyTypeIcon"),
	};

	static const FText Labels[] = {
		LOCTEXT("SingleVariable", "Single"),
		LOCTEXT("Array", "Array"),
		LOCTEXT("Set", "Set"),
		LOCTEXT("Map", "Map"),
	};

	static const FText Tooltips[] = {
		LOCTEXT("SingleVariableTooltip", "Single Variable"),
		LOCTEXT("ArrayTooltip", "Array"),
		LOCTEXT("SetTooltip", "Set"),
		LOCTEXT("MapTooltip", "Map (Dictionary)"),
	};
}


/** Wraps a custom pin type filter provided at construction time. */
class FPinTypeSelectorCustomFilterProxy : public IPinTypeSelectorFilter
{
public:
	FPinTypeSelectorCustomFilterProxy(TSharedRef<IPinTypeSelectorFilter> InFilter, FSimpleDelegate InOnFilterChanged)
		:Filter(InFilter)
	{
		// Auto-register the given delegate to respond to any filter change event and refresh the filtered item list, etc.
		OnFilterChanged_DelegateHandle = Filter->RegisterOnFilterChanged(InOnFilterChanged);
	}

	virtual ~FPinTypeSelectorCustomFilterProxy()
	{
		// Auto-unregister the delegate that was previously registered at construction time.
		Filter->UnregisterOnFilterChanged(OnFilterChanged_DelegateHandle);
	}

	virtual FDelegateHandle RegisterOnFilterChanged(FSimpleDelegate InOnFilterChanged)
	{
		return Filter->RegisterOnFilterChanged(InOnFilterChanged);
	}

	virtual void UnregisterOnFilterChanged(FDelegateHandle InDelegateHandle)
	{
		Filter->UnregisterOnFilterChanged(InDelegateHandle);
	}

	virtual TSharedPtr<SWidget> GetFilterOptionsWidget()
	{
		return Filter->GetFilterOptionsWidget();
	}

	virtual bool ShouldShowPinTypeTreeItem(FPinTypeTreeItem InItem) const
	{
		return Filter->ShouldShowPinTypeTreeItem(InItem);
	}

private:
	/** The underlying filter for which we're acting as a proxy. */
	TSharedRef<IPinTypeSelectorFilter> Filter;

	/** A handle to a delegate that gets called whenever the custom filter changes. Will be unregistered automatically when the proxy is destroyed. */
	FDelegateHandle OnFilterChanged_DelegateHandle;
};


class SPinTypeRow : public SComboRow<FPinTypeTreeItem>
{
public:
	SLATE_BEGIN_ARGS( SPinTypeRow )
	{}
		SLATE_DEFAULT_SLOT( FArguments, Content )
		SLATE_EVENT( FOnGetContent, OnGetMenuContent )
	SLATE_END_ARGS()
public:
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, TWeakPtr<SMenuOwner> InMenuOwner)
	{
		bHovered = false;
		SetHover(TAttribute<bool>::CreateSP(this, &SPinTypeRow::ShouldAppearHovered));

		SComboRow<FPinTypeTreeItem>::Construct( SComboRow<FPinTypeTreeItem>::FArguments()
			.ToolTip(InArgs._ToolTip)
			[
				SAssignNew(SubMenuHandler, SSubMenuHandler, InMenuOwner)
				.OnGetMenuContent(InArgs._OnGetMenuContent)
				.MenuContent(nullptr)
				[
					InArgs._Content.Widget
				]
			],
			InOwnerTable);
	}

	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		bHovered = true;
		SComboRow<FPinTypeTreeItem>::OnMouseEnter(MyGeometry, MouseEvent);
	}

	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override
	{
		bHovered = false;
		SComboRow<FPinTypeTreeItem>::OnMouseLeave(MouseEvent);
	}
	//~ End of SWidget interface

	/** Retuns TRUE if it should appear as hovered. */
	bool ShouldAppearHovered() const
	{
		return bHovered || SubMenuHandler.Pin()->ShouldSubMenuAppearHovered();
	}

	/** Returns TRUE if there is a Sub-Menu available to open */
	bool HasSubMenu() const
	{
		return SubMenuHandler.Pin()->HasSubMenu();
	}

	/** Returns TRUE if there is a Sub-Menu open */
	bool IsSubMenuOpen() const
	{
		return SubMenuHandler.Pin()->IsSubMenuOpen();
	}

	/** Forces the sub-menu open, clobbering any other open ones in the process */
	void RequestSubMenuToggle(bool bInImmediate = false)
	{
		SubMenuHandler.Pin()->RequestSubMenuToggle(true, true, bInImmediate);
	}

private:
	/** The Sub-MenuHandler which is managing the sub-menu content so that mousing over other rows will not close the sub-menus immediately */
	TWeakPtr<SSubMenuHandler> SubMenuHandler;
	/** Keep an internal IsHovered flag*/
	bool bHovered;
};

static bool ContainerRequiresGetTypeHash(EPinContainerType InType)
{
	return InType == EPinContainerType::Set || InType == EPinContainerType::Map;
}

static bool IsRightClickArrayStateToggleEnabled(SPinTypeSelector::ESelectorType SelectorType)
{
	// The Compact and Partial types both allow editing but omit the container-type
	// (e.g. value, array) control. For both of these types, enable right-clicking
	// on the type combo button to toggle between value/array.
	// The None type isn't editable, so it doesn't need this behavior.
	// The Full type shows a separate combo button for choosing the container type,
	// so it doesn't need this right-click behavior either.
	return SelectorType == SPinTypeSelector::ESelectorType::Compact ||
		SelectorType == SPinTypeSelector::ESelectorType::Partial;
}

TSharedRef<SWidget> SPinTypeSelector::ConstructPinTypeImage(const FSlateBrush* PrimaryIcon, const FSlateColor& PrimaryColor, const FSlateBrush* SecondaryIcon, const FSlateColor& SecondaryColor, TSharedPtr<SToolTip> InToolTip)
{
	return
		SNew(SLayeredImage, SecondaryIcon, SecondaryColor)
		.Image(PrimaryIcon)
		.ToolTip(InToolTip)
		.ColorAndOpacity(PrimaryColor);
}

TSharedRef<SWidget> SPinTypeSelector::ConstructPinTypeImage(TAttribute<const FSlateBrush*> PrimaryIcon, TAttribute<FSlateColor> PrimaryColor, TAttribute<const FSlateBrush*> SecondaryIcon, TAttribute<FSlateColor> SecondaryColor )
{
	return
		SNew(SLayeredImage, SecondaryIcon, SecondaryColor)
		.Image(PrimaryIcon)
		.ColorAndOpacity(PrimaryColor);
}

TSharedRef<SWidget> SPinTypeSelector::ConstructPinTypeImage(UEdGraphPin* Pin)
{
	// Color and image bindings:
	TAttribute<const FSlateBrush*> PrimaryIcon = TAttribute<const FSlateBrush*>::Create(
		TAttribute<const FSlateBrush*>::FGetter::CreateLambda(
			[Pin]() -> const FSlateBrush*
			{
				if(!Pin->IsPendingKill())
				{
					return FBlueprintEditorUtils::GetIconFromPin(Pin->PinType, /* bIsLarge = */true);
				}
				return nullptr;
			}
		)
	);

	TAttribute<FSlateColor> PrimaryColor = TAttribute<FSlateColor>::Create(
		TAttribute<FSlateColor>::FGetter::CreateLambda(
			[Pin]() -> FSlateColor
			{
				if(!Pin->IsPendingKill())
				{
					if(const UEdGraphSchema_K2* PCSchema = Cast<UEdGraphSchema_K2>(Pin->GetSchema()))
					{
						FLinearColor PrimaryLinearColor = PCSchema->GetPinTypeColor(Pin->PinType);
						PrimaryLinearColor.A = .25f;
						return PrimaryLinearColor;
					}
				}
				return FSlateColor(FLinearColor::White);
			}
		)
	);
	TAttribute<const FSlateBrush*> SecondaryIcon = TAttribute<const FSlateBrush*>::Create(
		TAttribute<const FSlateBrush*>::FGetter::CreateLambda(
			[Pin]() -> const FSlateBrush*
			{
				if(!Pin->IsPendingKill())
				{
					return FBlueprintEditorUtils::GetSecondaryIconFromPin(Pin->PinType);
				}
				return nullptr;
			}
		)
	);
	
	TAttribute<FSlateColor> SecondaryColor = TAttribute<FSlateColor>::Create(
		TAttribute<FSlateColor>::FGetter::CreateLambda(
			[Pin]() -> FSlateColor
			{
				if(!Pin->IsPendingKill())
				{
					if(const UEdGraphSchema_K2* SCSchema = Cast<UEdGraphSchema_K2>(Pin->GetSchema()))
					{
						FLinearColor SecondaryLinearColor = SCSchema->GetSecondaryPinTypeColor(Pin->PinType);
						SecondaryLinearColor.A = .25f;
						return SecondaryLinearColor;
					}
				}
				return FSlateColor(FLinearColor::White);
			}
		)
	);

	return ConstructPinTypeImage(PrimaryIcon, PrimaryColor, SecondaryIcon, SecondaryColor);
}

void SPinTypeSelector::Construct(const FArguments& InArgs, FGetPinTypeTree GetPinTypeTreeFunc)
{
	SearchText = FText::GetEmpty();

	ReadOnly = InArgs._ReadOnly;

	OnTypeChanged = InArgs._OnPinTypeChanged;
	OnTypePreChanged = InArgs._OnPinTypePreChanged;

	check(GetPinTypeTreeFunc.IsBound());
	GetPinTypeTree = GetPinTypeTreeFunc;

	Schema = InArgs._Schema;
	SchemaAction = InArgs._SchemaAction;
	TypeTreeFilter = InArgs._TypeTreeFilter;
	TreeViewWidth = InArgs._TreeViewWidth;
	TreeViewHeight = InArgs._TreeViewHeight;

	TargetPinType = InArgs._TargetPinType;
	SelectorType = InArgs._SelectorType;

	NumFilteredPinTypeItems = 0;
	NumValidPinTypeItems = 0;

	if(InArgs._CustomFilters.Num() > 0)
	{
		for(const TSharedPtr<IPinTypeSelectorFilter>& Filter : InArgs._CustomFilters)
		{
			CustomFilters.Add(MakeShared<FPinTypeSelectorCustomFilterProxy>(Filter.ToSharedRef(), FSimpleDelegate::CreateSP(this, &SPinTypeSelector::OnCustomFilterChanged)));
		}
	}
	else if (InArgs._CustomFilter.IsValid())
	{
		CustomFilters.Add(MakeShared<FPinTypeSelectorCustomFilterProxy>(InArgs._CustomFilter.ToSharedRef(), FSimpleDelegate::CreateSP(this, &SPinTypeSelector::OnCustomFilterChanged)));
	}
	else if (UClass* PinTypeSelectorFilterClass = GetDefault<UPinTypeSelectorFilter>()->FilterClass.LoadSynchronous() )
	{
		TSharedPtr<IPinTypeSelectorFilter> SelectorFilter = GetDefault<UPinTypeSelectorFilter>(PinTypeSelectorFilterClass)->GetPinTypeSelectorFilter();
		CustomFilters.Add(MakeShared<FPinTypeSelectorCustomFilterProxy>(SelectorFilter.ToSharedRef(), FSimpleDelegate::CreateSP(this, &SPinTypeSelector::OnCustomFilterChanged)));
	}

	bIsRightMousePressed = false;

	// Depending on whether this is a full selector or not, we generate a different primary type image widget
	TSharedPtr<SWidget> PrimaryTypeImage;
	if (SelectorType == ESelectorType::Full)
	{
		// Full selector displays container and secondary type separately, so the main combo button should just have the primary icon
		PrimaryTypeImage =
			SNew(SImage)
			.Image(this, &SPinTypeSelector::GetTypeIconImage)
			.ColorAndOpacity(this, &SPinTypeSelector::GetTypeIconColor);
	}
	else
	{
		// Partial/compact selectors do not display container or secondary type separately, so we need to jam it all in the one image
		PrimaryTypeImage =
			SNew(
				SLayeredImage,
				TAttribute<const FSlateBrush*>(this, &SPinTypeSelector::GetSecondaryTypeIconImage),
				TAttribute<FSlateColor>(this, &SPinTypeSelector::GetSecondaryTypeIconColor)
			)
			.Image(this, &SPinTypeSelector::GetTypeIconImage)
			.ColorAndOpacity(this, &SPinTypeSelector::GetTypeIconColor);
	}

	// Depending on if this is a compact selector or not, we generate a different compound widget
	TSharedPtr<SWidget> Widget;
	const TSharedPtr<SWidget> ReadOnlyWidget = SNew(SHorizontalBox)
			.Clipping(EWidgetClipping::OnDemand)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.Padding(FMargin(2.0f, 3.0f, 2.0f, 3.0f))
			.AutoWidth()
			[
				// Read-only version does not display container or secondary type separately, so we need to jam it all in the one image
				SNew(SLayeredImage, TAttribute<const FSlateBrush*>(this, &SPinTypeSelector::GetSecondaryTypeIconImage), TAttribute<FSlateColor>(this, &SPinTypeSelector::GetSecondaryTypeIconColor))
				.Image(this, &SPinTypeSelector::GetTypeIconImage)
				.ColorAndOpacity(this, &SPinTypeSelector::GetTypeIconColor)
			]
			+ SHorizontalBox::Slot()
			.Padding(2.0f, 2.0f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(this, &SPinTypeSelector::GetTypeDescription, false)
				.Font(InArgs._Font)
				.ColorAndOpacity(FSlateColor::UseForeground())
			];

	if (SelectorType == ESelectorType::Compact)
	{
		// Only have a combo button with an icon
		Widget = SAssignNew( TypeComboButton, SComboButton )
			.OnGetMenuContent(this, &SPinTypeSelector::GetMenuContent, false)
			.ContentPadding(0.0f)
			.ToolTipText(this, &SPinTypeSelector::GetToolTipForComboBoxType)
			.HasDownArrow(false)
			.ButtonStyle(FAppStyle::Get(),  "BlueprintEditor.CompactPinTypeSelector")
			.ButtonContent()
			[
				PrimaryTypeImage.ToSharedRef()
			];
	}
	else if (SelectorType == ESelectorType::None)
	{
		Widget = ReadOnlyWidget;
	}
	else if (SelectorType == ESelectorType::Full || SelectorType == ESelectorType::Partial)
	{
		TSharedPtr<SWidget> ContainerControl;

		if(SelectorType == ESelectorType::Full)
		{
			// Traditional Pin Type Selector with a combo button, the icon, the current type name, and a toggle button for being an array
			ContainerControl = SNew(SComboButton)
				.ComboButtonStyle(FAppStyle::Get(),"BlueprintEditor.CompactVariableTypeSelector")
				.MenuPlacement(EMenuPlacement::MenuPlacement_ComboBoxRight)
				.OnGetMenuContent(this, &SPinTypeSelector::GetPinContainerTypeMenuContent)
				.ContentPadding(0.0f)
				.ToolTip(IDocumentation::Get()->CreateToolTip(TAttribute<FText>(this, &SPinTypeSelector::GetToolTipForContainerWidget), NULL, *PinTypeSelectorStatics::BigTooltipDocLink, TEXT("Containers")))
				.IsEnabled(TargetPinType.Get().PinCategory != UEdGraphSchema_K2::PC_Exec)
				.Visibility(InArgs._bAllowArrays ? EVisibility::Visible : EVisibility::Collapsed)
				.ButtonContent()
				[
					SNew(SLayeredImage, TAttribute<const FSlateBrush*>(this, &SPinTypeSelector::GetSecondaryTypeIconImage), TAttribute<FSlateColor>(this, &SPinTypeSelector::GetSecondaryTypeIconColor))
					.Image(this, &SPinTypeSelector::GetTypeIconImage)
					.ColorAndOpacity(this, &SPinTypeSelector::GetTypeIconColor)
				];
		}

		TSharedRef<SHorizontalBox> HBox = SNew(SHorizontalBox).Clipping(EWidgetClipping::ClipToBoundsAlways);
		Widget = HBox;

		const float FullComboButtonWidth = 125.f;

		HBox->AddSlot()
		.HAlign(HAlign_Left)
		[
			SNew(SBox)
			.WidthOverride(SelectorType == ESelectorType::Full ? FullComboButtonWidth : FOptionalSize())
			[
				SAssignNew(TypeComboButton, SComboButton)
				.ComboButtonStyle(FAppStyle::Get(), "ComboButton")
				.OnGetMenuContent(this, &SPinTypeSelector::GetMenuContent, false)
				.ContentPadding(0.0f)
				.ToolTipText(this, &SPinTypeSelector::GetToolTipForComboBoxType)
				.ForegroundColor(FSlateColor::UseForeground())
				.ButtonContent()
				[
					SNew(SHorizontalBox)
					.Clipping(EWidgetClipping::ClipToBoundsAlways)
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Left)
					.Padding(0.0f, 0.0f, 2.0f, 0.0f)
					.AutoWidth()
					[
						PrimaryTypeImage.ToSharedRef()
					]
					+ SHorizontalBox::Slot()
					.Padding(2.0f, 0.0f, 0.0f, 0.0f)
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Left)
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(this, &SPinTypeSelector::GetTypeDescription, false)
						.Font(InArgs._Font)
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
			]
		];

		if(SelectorType == ESelectorType::Full)
		{
			HBox->AddSlot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.Padding(2.0f)
				[
					ContainerControl.ToSharedRef()
				];
	
			HBox->AddSlot()
			[
				SNew(SBox)
				.WidthOverride(FullComboButtonWidth)
				.Visibility_Lambda([this]() {return this->TargetPinType.Get().IsMap() == true ? EVisibility::Visible : EVisibility::Collapsed; })
				[
					SAssignNew( SecondaryTypeComboButton, SComboButton )
					.OnGetMenuContent(this, &SPinTypeSelector::GetMenuContent, true )
					.ContentPadding(0.0f)
					.ToolTipText(this, &SPinTypeSelector::GetToolTipForComboBoxSecondaryType)
					.ButtonContent()
					[
						SNew(SHorizontalBox)
						.Clipping(EWidgetClipping::OnDemand)
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Left)
						.Padding(0.0f, 0.0f, 2.0f, 0.0f)
						[
							SNew(SImage)
							.Image( this, &SPinTypeSelector::GetSecondaryTypeIconImage )
							.ColorAndOpacity( this, &SPinTypeSelector::GetSecondaryTypeIconColor )
						]
						+SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Left)
						.Padding(2.0f, 0.0f, 0.0f, 0.0f)
						[
							SNew(STextBlock)
							.Text(this, &SPinTypeSelector::GetSecondaryTypeDescription, false)
							.Font(InArgs._Font)
						]
					]
				]
			];
		}
	}


	this->ChildSlot
	[
		SNew(SWidgetSwitcher)
		// Don't show the read-only widget when in Compact mode
		.WidgetIndex_Lambda([this](){return ReadOnly.Get() && SelectorType != ESelectorType::Compact ? 1 : 0; })
		+ SWidgetSwitcher::Slot() // editable version
		// Apply additional padding to Partial and Full selector type to prevent padding jumps
		.Padding(SelectorType == ESelectorType::Partial || SelectorType == ESelectorType::Full ? FMargin(-6.0f, 0.0f,0.0f,0.0f) : FMargin(0))
		[
			Widget.ToSharedRef()
		]
		+ SWidgetSwitcher::Slot() // read-only version
		[
			ReadOnlyWidget.ToSharedRef()
		]	
	];
}

//=======================================================================
// Attribute Helpers

FText SPinTypeSelector::GetTypeDescription(const bool bIncludeSubcategory) const
{
	const FName PinSubCategory = TargetPinType.Get().PinSubCategory;
	const UObject* PinSubCategoryObject = TargetPinType.Get().PinSubCategoryObject.Get();
	if (PinSubCategory != UEdGraphSchema_K2::PSC_Bitmask && PinSubCategoryObject)
	{
		if (const UField* Field = Cast<const UField>(PinSubCategoryObject))
		{
			return Field->GetDisplayNameText();
		}
		return FText::FromString(PinSubCategoryObject->GetName());
	}
	else
	{
		return UEdGraphSchema_K2::GetCategoryText(TargetPinType.Get().PinCategory, bIncludeSubcategory ? PinSubCategory : NAME_None, true);
	}
}

FText SPinTypeSelector::GetSecondaryTypeDescription(const bool bIncludeSubcategory) const
{
	const FName PinSubCategory = TargetPinType.Get().PinValueType.TerminalSubCategory;
	const UObject* PinSubCategoryObject = TargetPinType.Get().PinValueType.TerminalSubCategoryObject.Get();
	if (PinSubCategory != UEdGraphSchema_K2::PSC_Bitmask && PinSubCategoryObject)
	{
		if (auto Field = Cast<const UField>(PinSubCategoryObject))
		{
			return Field->GetDisplayNameText();
		}
		return FText::FromString(PinSubCategoryObject->GetName());
	}
	else
	{
		return UEdGraphSchema_K2::GetCategoryText(TargetPinType.Get().PinValueType.TerminalCategory, bIncludeSubcategory ? PinSubCategory : NAME_None, true);
	}
}

FText SPinTypeSelector::GetCombinedTypeDescription(const bool bIncludeSubcategory) const
{
	FFormatNamedArguments Args;

	EPinContainerType ContainerType = TargetPinType.Get().ContainerType;
	switch (TargetPinType.Get().ContainerType)
	{
	case EPinContainerType::Map:
		Args.Add(TEXT("KeyTitle"), GetTypeDescription(bIncludeSubcategory));
		Args.Add(TEXT("ValueTitle"), GetSecondaryTypeDescription(bIncludeSubcategory));
		return FText::Format(NSLOCTEXT("SPinTypeSelector", "MapAsText", "Map of {KeyTitle} types to {ValueTitle} types"), Args);
	case EPinContainerType::Set:
		Args.Add(TEXT("PropertyTitle"), GetTypeDescription(bIncludeSubcategory));
		return FText::Format(NSLOCTEXT("SPinTypeSelector", "SetAsText", "Set of {PropertyTitle} types"), Args);
	case EPinContainerType::Array:
		Args.Add(TEXT("PropertyTitle"), GetTypeDescription(bIncludeSubcategory));
		return FText::Format(NSLOCTEXT("SPinTypeSelector", "ArrayAsText", "Array of {PropertyTitle} types"), Args);
	default:
		Args.Add(TEXT("PropertyTitle"), GetTypeDescription(bIncludeSubcategory));
		return FText::Format(NSLOCTEXT("SPinTypeSelector", "PrimitiveAsText", "{PropertyTitle} type"), Args);
	}
}

const FSlateBrush* SPinTypeSelector::GetTypeIconImage() const
{
	return FBlueprintEditorUtils::GetIconFromPin(TargetPinType.Get());
}

const FSlateBrush* SPinTypeSelector::GetSecondaryTypeIconImage() const
{
	return FBlueprintEditorUtils::GetSecondaryIconFromPin(TargetPinType.Get());
}

FSlateColor SPinTypeSelector::GetTypeIconColor() const
{
	return Schema->GetPinTypeColor(TargetPinType.Get());
}

FSlateColor SPinTypeSelector::GetSecondaryTypeIconColor() const
{
	return Schema->GetSecondaryPinTypeColor(TargetPinType.Get());
}

ECheckBoxState SPinTypeSelector::IsArrayChecked() const
{
	return TargetPinType.Get().IsArray() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SPinTypeSelector::OnArrayCheckStateChanged(ECheckBoxState NewState)
{
	FEdGraphPinType NewTargetPinType = TargetPinType.Get();
	NewTargetPinType.ContainerType = (NewState == ECheckBoxState::Checked) ? EPinContainerType::Array : EPinContainerType::None;

	OnTypeChanged.ExecuteIfBound(NewTargetPinType);
}

void SPinTypeSelector::OnArrayStateToggled()
{
	OnArrayCheckStateChanged((IsArrayChecked() == ECheckBoxState::Checked)? ECheckBoxState::Unchecked : ECheckBoxState::Checked);
}

void SPinTypeSelector::OnContainerTypeSelectionChanged( EPinContainerType PinContainerType)
{
	const FScopedTransaction Transaction(LOCTEXT("ChangeParam", "Change Parameter Type"));

	FEdGraphPinType NewTargetPinType = TargetPinType.Get();
	NewTargetPinType.ContainerType = PinContainerType;

	OnTypeChanged.ExecuteIfBound(NewTargetPinType);
}

//=======================================================================
// Type TreeView Support
TSharedRef<ITableRow> SPinTypeSelector::GenerateTypeTreeRow(FPinTypeTreeItem InItem, const TSharedRef<STableViewBase>& OwnerTree, bool bForSecondaryType)
{
	const bool bHasChildren = (InItem->Children.Num() > 0);
	const FText Description = InItem->GetDescription();
	const FEdGraphPinType& PinType = InItem->GetPinType(false);

	// Determine the best icon the to represents this item
	const FSlateBrush* IconBrush = FBlueprintEditorUtils::GetIconFromPin(PinType);

	// Use tooltip if supplied, otherwise just repeat description
	const FText OrgTooltip = InItem->GetToolTip();
	FText Tooltip = !OrgTooltip.IsEmpty() ? OrgTooltip : Description;

	// Switch to short tooltip based on settings
	UClass* ClassType = Cast<UClass>(PinType.PinSubCategoryObject);
	if (ClassType && ClassType->IsNative() && GetDefault<UBlueprintEditorSettings>()->bShowShortTooltips)
	{
		Tooltip = ClassType->GetToolTipText(true);
	}

	// If this is a struct type, get some useful information about it's native C++ declaration
	if (PinType.PinCategory == UEdGraphSchema_K2::PC_Struct)
	{
		UScriptStruct* StructType = Cast<UScriptStruct>(PinType.PinSubCategoryObject);
		if (StructType && StructType->IsNative())
		{
			Tooltip = FText::Format(LOCTEXT("NativePinTypeName", "{0}\n\n@see {1}"),
				Tooltip,
				FText::FromString(StructType->GetStructCPPName())
			);
		}
	}

	const FString PinTooltipExcerpt = ((PinType.PinCategory != UEdGraphSchema_K2::PC_Byte || PinType.PinSubCategoryObject == nullptr) ? PinType.PinCategory.ToString() : TEXT("Enum")); 

	// If there is a sub-menu for this pin type, we need to bind the function to handle the sub-menu
	FOnGetContent OnGetContent;
	if (InItem->GetPossibleObjectReferenceTypes() != static_cast<uint8>(EObjectReferenceType::NotAnObject))
	{
		OnGetContent.BindSP(this, &SPinTypeSelector::GetAllowedObjectTypes, InItem, bForSecondaryType);
	}

	TSharedPtr< SHorizontalBox > HorizontalBox;
	TSharedRef< ITableRow > ReturnWidget = SNew( SPinTypeRow, OwnerTree, MenuContent )
		.ToolTip( IDocumentation::Get()->CreateToolTip( Tooltip, NULL, *PinTypeSelectorStatics::BigTooltipDocLink, PinTooltipExcerpt) )
		.OnGetMenuContent(OnGetContent)
		[
			SAssignNew(HorizontalBox, SHorizontalBox)
			+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(1.f)
			[
				SNew(SImage)
					.Image(IconBrush)
					.ColorAndOpacity(Schema->GetPinTypeColor(PinType))
					.Visibility( InItem->bReadOnly ? EVisibility::Collapsed : EVisibility::Visible )
			]
			+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(1.f)
			[
				SNew(STextBlock)
					.Text(Description)
					.HighlightText(SearchText)
					.Font( bHasChildren ? FAppStyle::GetFontStyle(TEXT("Kismet.TypePicker.CategoryFont")) : FAppStyle::GetFontStyle(TEXT("Kismet.TypePicker.NormalFont")) )
			]
		];

	// Add a sub-menu indicator arrow to inform the user that there are sub-items to be displayed
	if (OnGetContent.IsBound())
	{
		HorizontalBox->AddSlot()
			.FillWidth(1.0f)
			.VAlign( VAlign_Center )
			.HAlign( HAlign_Right )
			[
				SNew( SBox )
				.Padding(FMargin(7,0,0,0))
				[
					SNew( SImage )
					.Image( FAppStyle::Get().GetBrush( "Menu.SubMenuIndicator" ) )
				]
			];
	}

	return ReturnWidget;
}

TSharedRef<SWidget> SPinTypeSelector::CreateObjectReferenceWidget(FPinTypeTreeItem InItem, FEdGraphPinType& InPinType, const FSlateBrush* InIconBrush, FText InSimpleTooltip) const
{
	return SNew(SHorizontalBox)
		.ToolTip(IDocumentation::Get()->CreateToolTip(InSimpleTooltip, nullptr, *PinTypeSelectorStatics::BigTooltipDocLink, InPinType.PinCategory.ToString()))
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(1.f)
		[
			SNew(SImage)
			.Image(InIconBrush)
			.ColorAndOpacity(Schema->GetPinTypeColor(InPinType))
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(1.f)
		[
			SNew(STextBlock)
			.Text(UEdGraphSchema_K2::GetCategoryText(InPinType.PinCategory))
			.Font(FAppStyle::GetFontStyle(TEXT("Kismet.TypePicker.NormalFont")) )
		];
}

class SObjectReferenceWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SObjectReferenceWidget )
	{}
		SLATE_DEFAULT_SLOT( FArguments, Content )
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TWeakPtr< SMenuOwner > InMenuOwner)
	{
		MenuOwner = InMenuOwner;

		this->ChildSlot
		[
			InArgs._Content.Widget
		];
	}

	// SWidget interface
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& KeyEvent) override
	{
		if (MenuOwner.IsValid() && (KeyEvent.GetKey() == EKeys::Left || KeyEvent.GetKey() == EKeys::Escape))
		{
			MenuOwner.Pin()->CloseSummonedMenus();
			return FReply::Handled();
		}
		return FReply::Unhandled();
	}
	// End of SWidget interface

private:
	TWeakPtr< SMenuOwner > MenuOwner;
};

TSharedRef<ITableRow> SPinTypeSelector::GenerateObjectReferenceTreeRow(FObjectReferenceListItem InItem, const TSharedRef<STableViewBase>& OwnerTree)
{
	return SNew(SComboRow<FObjectReferenceListItem>, OwnerTree)
		[
			InItem->WidgetToDisplay.ToSharedRef()
		];
}

void SPinTypeSelector::OnObjectReferenceSelectionChanged(FObjectReferenceListItem InItem, ESelectInfo::Type SelectInfo, bool bForSecondaryType)
{
	if (SelectInfo != ESelectInfo::OnNavigation)
	{
		OnSelectPinType(InItem->PinTypeItem, InItem->PinCategory, bForSecondaryType);
	}
}

TSharedRef< SWidget > SPinTypeSelector::GetAllowedObjectTypes(FPinTypeTreeItem InItem, bool bForSecondaryType)
{
	GenerateAllowedObjectTypesList(AllowedObjectReferenceTypes, InItem, bForSecondaryType);

	TSharedRef<SListView<FObjectReferenceListItem>> ListView = SNew(SListView<FObjectReferenceListItem>)
		.ListItemsSource(&AllowedObjectReferenceTypes)
		.SelectionMode(ESelectionMode::Single)
		.OnGenerateRow(this, &SPinTypeSelector::GenerateObjectReferenceTreeRow)
		.OnSelectionChanged(this, &SPinTypeSelector::OnObjectReferenceSelectionChanged, bForSecondaryType);

	WeakListView = ListView;

	if (AllowedObjectReferenceTypes.Num())
	{
		ListView->SetSelection(AllowedObjectReferenceTypes[0], ESelectInfo::OnNavigation);
	}

	return 
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		[
			SNew(SObjectReferenceWidget, PinTypeSelectorMenuOwner)
			[
				SNew(SListViewSelectorDropdownMenu<FObjectReferenceListItem>, nullptr, ListView)
				[
					ListView
				]
			]
		];
}

void SPinTypeSelector::GenerateAllowedObjectTypesList(TArray<FObjectReferenceListItem>& OutList, FPinTypeTreeItem InItem, bool bForSecondaryType) const
{
	OutList.Reset();

	// Do not force the pin type here, that causes a load of the Blueprint (if unloaded)
	FEdGraphPinType PinType = InItem->GetPinType(false);
	const FSlateBrush* IconBrush = FBlueprintEditorUtils::GetIconFromPin(PinType);

	FFormatNamedArguments Args;
	bool bIsNative = false;

	if (PinType.PinSubCategory != UEdGraphSchema_K2::PSC_Bitmask && PinType.PinSubCategoryObject.IsValid())
	{
		bIsNative = PinType.PinSubCategoryObject->IsNative() && PinType.PinSubCategoryObject != UBlueprint::StaticClass();
		Args.Add(TEXT("TypeName"), InItem->GetDescription());
	}

	uint8 PossibleObjectReferenceTypes = InItem->GetPossibleObjectReferenceTypes();

	// Per each object reference type, change the category to the type and add a menu entry (this will get the color to be correct)

	if (PossibleObjectReferenceTypes & static_cast<uint8>(EObjectReferenceType::ObjectReference))
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_Object;
		TSharedRef<SWidget> Widget = CreateObjectReferenceWidget(InItem, PinType, IconBrush, FText::Format(LOCTEXT("ObjectTooltip", "Reference an instanced object of type \'{TypeName}\'"), Args));
		FObjectReferenceListItem ObjectReferenceType = MakeShareable(new FObjectReferenceType(InItem, Widget, PinType.PinCategory));
		OutList.Add(ObjectReferenceType);
	}

	if (PossibleObjectReferenceTypes & static_cast<uint8>(EObjectReferenceType::ClassReference))
	{
		PinType.PinCategory = UEdGraphSchema_K2::PC_Class;
		TSharedRef<SWidget> Widget = CreateObjectReferenceWidget(InItem, PinType, IconBrush, FText::Format(LOCTEXT("ClassTooltip", "Reference a class inheriting from type \'{TypeName}\'"), Args));
		FObjectReferenceListItem ObjectReferenceType = MakeShareable(new FObjectReferenceType(InItem, Widget, PinType.PinCategory));
		OutList.Add(ObjectReferenceType);
	}

	if (PossibleObjectReferenceTypes & static_cast<uint8>(EObjectReferenceType::SoftObject))
	{
		FText FormatString = LOCTEXT("AssetTooltip", "Path to an instanced object of type \'{TypeName}\' which may not be loaded. Can be used to asynchronously load the asset.");
		if (!bIsNative)
		{
			FormatString = LOCTEXT("AssetBPTooltip", "Path to an instanced object of type \'{TypeName}\' which may not be loaded. The blueprint type itself will always be loaded, which can be expensive.");
		}

		PinType.PinCategory = UEdGraphSchema_K2::PC_SoftObject;
		TSharedRef<SWidget> Widget = CreateObjectReferenceWidget(InItem, PinType, IconBrush, FText::Format(FormatString, Args));
		FObjectReferenceListItem ObjectReferenceType = MakeShareable(new FObjectReferenceType(InItem, Widget, PinType.PinCategory));
		OutList.Add(ObjectReferenceType);
	}

	if (PossibleObjectReferenceTypes & static_cast<uint8>(EObjectReferenceType::SoftClass))
	{
		FText FormatString = LOCTEXT("ClassAssetTooltip", "Path to a class inheriting from type \'{TypeName}\' which may not be loaded. Can be utilized to asynchronously load the class.");
		if (!bIsNative)
		{
			FormatString = LOCTEXT("ClassAssetBPTooltip", "Path to a class inheriting from type \'{TypeName}\' which may not be loaded. The blueprint type itself will always be loaded, which can be expensive.");
		}

		PinType.PinCategory = UEdGraphSchema_K2::PC_SoftClass;
		TSharedRef<SWidget> Widget = CreateObjectReferenceWidget(InItem, PinType, IconBrush, FText::Format(FormatString, Args));
		FObjectReferenceListItem ObjectReferenceType = MakeShareable(new FObjectReferenceType(InItem, Widget, PinType.PinCategory));
		OutList.Add(ObjectReferenceType);
	}
}

void SPinTypeSelector::OnSelectPinType(FPinTypeTreeItem InItem, FName InPinCategory, bool bForSecondaryType)
{
	const FScopedTransaction Transaction( LOCTEXT("ChangeParam", "Change Parameter Type") );

	FEdGraphPinType NewTargetPinType = TargetPinType.Get();
	//Call delegate in order to notify pin type change is about to happen
	OnTypePreChanged.ExecuteIfBound(NewTargetPinType);

	const FEdGraphPinType& SelectionPinType = InItem->GetPinType(true);

	// Change the pin's type
	if (bForSecondaryType)
	{
		NewTargetPinType.PinValueType.TerminalCategory = InPinCategory;
		NewTargetPinType.PinValueType.TerminalSubCategory = SelectionPinType.PinSubCategory;
		NewTargetPinType.PinValueType.TerminalSubCategoryObject = SelectionPinType.PinSubCategoryObject;
	}
	else
	{
		NewTargetPinType.PinCategory = InPinCategory;
		NewTargetPinType.PinSubCategory = SelectionPinType.PinSubCategory;
		NewTargetPinType.PinSubCategoryObject = SelectionPinType.PinSubCategoryObject;
	}


	TypeComboButton->SetIsOpen(false);
	if (SecondaryTypeComboButton.IsValid())
	{
		SecondaryTypeComboButton->SetIsOpen(false);
	}

	if( NewTargetPinType.PinCategory == UEdGraphSchema_K2::PC_Exec )
	{
		NewTargetPinType.ContainerType = EPinContainerType::None;
		NewTargetPinType.PinValueType.TerminalCategory = NAME_None;
		NewTargetPinType.PinValueType.TerminalSubCategory = NAME_None;
		NewTargetPinType.PinValueType.TerminalSubCategoryObject = nullptr;
	}

	if ((NewTargetPinType.IsMap() || NewTargetPinType.IsSet()) && !FBlueprintEditorUtils::HasGetTypeHash(NewTargetPinType))
	{
		FEdGraphPinType HashedType = NewTargetPinType;
		// clear the container-ness for messaging, we want to explain that the contained type is not hashable,
		// not message about the container type (e.g. "Container type cleared because 'bool' does not have a GetTypeHash..."
		// instead of "Container Type cleared because 'map of bool to float'..."). We also need to clear this because
		// the type cannot be a container:
		NewTargetPinType.ContainerType = EPinContainerType::None;

		// inform user via toast why the type change was exceptional and clear IsMap/IsSetness because this type cannot be hashed:
		const FText NotificationText = FText::Format(LOCTEXT("TypeCannotBeHashed", "Container type cleared because '{0}' does not have a GetTypeHash function. Maps and Sets require a hash function to insert and find elements"), UEdGraphSchema_K2::TypeToText(NewTargetPinType));
		FNotificationInfo Info(NotificationText);
		Info.FadeInDuration = 0.0f;
		Info.FadeOutDuration = 0.0f;
		Info.ExpireDuration = 10.0f;
		FSlateNotificationManager::Get().AddNotification(Info);
	}

	if (!Schema->SupportsPinTypeContainer(SchemaAction, NewTargetPinType, NewTargetPinType.ContainerType))
	{
		NewTargetPinType.ContainerType = EPinContainerType::None;
	}
	
	SearchText = FText::GetEmpty();

	OnTypeChanged.ExecuteIfBound(NewTargetPinType);
}

void SPinTypeSelector::OnTypeSelectionChanged(FPinTypeTreeItem Selection, ESelectInfo::Type SelectInfo, bool bForSecondaryType)
{
	// When the user is navigating, do not act upon the selection change
	if(SelectInfo == ESelectInfo::OnNavigation)
	{
		// Unless mouse clicking on an item with a sub-menu, all attempts to auto-select should open the sub-menu
		TSharedPtr<SPinTypeRow> PinRow = StaticCastSharedPtr<SPinTypeRow>(TypeTreeView->WidgetFromItem(Selection));
		if (PinRow.IsValid() && PinTypeSelectorMenuOwner.IsValid())
		{
			PinTypeSelectorMenuOwner.Pin()->CloseSummonedMenus();
		}
		return;
	}

	// Only handle selection for non-read only items, since STreeViewItem doesn't actually support read-only
	if( Selection.IsValid() )
	{
		if( !Selection->bReadOnly )
		{
			// Unless mouse clicking on an item with a sub-menu, all attempts to auto-select should open the sub-menu
			TSharedPtr<SPinTypeRow> PinRow = StaticCastSharedPtr<SPinTypeRow>(TypeTreeView->WidgetFromItem(Selection));
			if (SelectInfo != ESelectInfo::OnMouseClick && PinRow.IsValid() && PinRow->HasSubMenu() && !PinRow->IsSubMenuOpen())
			{
				PinRow->RequestSubMenuToggle(true);
				FSlateApplication::Get().SetKeyboardFocus(WeakListView.Pin(), EFocusCause::SetDirectly);
			}
			else
			{
				OnSelectPinType(Selection, Selection->GetPossibleObjectReferenceTypes() == static_cast<uint8>(EObjectReferenceType::AllTypes)? UEdGraphSchema_K2::PC_Object : Selection->GetPinType(false).PinCategory, bForSecondaryType);
			}
		}
		else
		{
			// Expand / contract the category, if applicable
			if( Selection->Children.Num() > 0 )
			{
				const bool bIsExpanded = TypeTreeView->IsItemExpanded(Selection);
				TypeTreeView->SetItemExpansion(Selection, !bIsExpanded);

				if(SelectInfo == ESelectInfo::OnMouseClick)
				{
					TypeTreeView->ClearSelection();
				}
			}
		}
	}
}

void SPinTypeSelector::GetTypeChildren(FPinTypeTreeItem InItem, TArray<FPinTypeTreeItem>& OutChildren)
{
	OutChildren = InItem->Children;
}

TSharedRef<SWidget>	SPinTypeSelector::GetMenuContent(bool bForSecondaryType)
{
	GetPinTypeTree.Execute(TypeTreeRoot, TypeTreeFilter);

	// Remove types not supported by schema
	FilterUnsupportedTypes(TypeTreeRoot);

	if( !MenuContent.IsValid() || (bForSecondaryType != bMenuContentIsSecondary) || (MenuContextPinType != TargetPinType.Get()) )
	{
		bMenuContentIsSecondary = bForSecondaryType;
		// Pre-build the tree view and search box as it is needed as a parameter for the context menu's container.
		SAssignNew(TypeTreeView, SPinTypeTreeView)
			.TreeItemsSource(&FilteredTypeTreeRoot)
			.SelectionMode(ESelectionMode::Single)
			.OnGenerateRow( this, &SPinTypeSelector::GenerateTypeTreeRow, bForSecondaryType )
			.OnSelectionChanged( this, &SPinTypeSelector::OnTypeSelectionChanged, bForSecondaryType )
			.OnGetChildren(this, &SPinTypeSelector::GetTypeChildren);

		SAssignNew(FilterTextBox, SSearchBox)
			.InitialText(SearchText)
			.OnTextChanged( this, &SPinTypeSelector::OnFilterTextChanged )
			.OnTextCommitted( this, &SPinTypeSelector::OnFilterTextCommitted );

		TArray<TSharedPtr<SWidget>> CustomFilterOptionsWidgets;
		for(const TSharedPtr<IPinTypeSelectorFilter>& CustomFilter : CustomFilters)
		{
			if(CustomFilter.IsValid())
			{
				TSharedPtr<SWidget> CustomWidget = CustomFilter->GetFilterOptionsWidget();
				if(CustomWidget.IsValid())
				{
					CustomFilterOptionsWidgets.Add(CustomWidget);
				}
			}
		}

		TSharedPtr<SHorizontalBox> CustomWidgetContainer;
		TSharedPtr<SVerticalBox> TreeWrapper;

		MenuContextPinType = TargetPinType.Get();
		MenuContent = SAssignNew(PinTypeSelectorMenuOwner, SMenuOwner)
			[
				SAssignNew(TreeWrapper, SVerticalBox)
				+SVerticalBox::Slot()
				.FillHeight(1.f)
				[
					SNew(SListViewSelectorDropdownMenu<FPinTypeTreeItem>, FilterTextBox, TypeTreeView)
					[
						SNew( SVerticalBox )
						+SVerticalBox::Slot()
						.AutoHeight()
						.Padding(4.f, 4.f, 4.f, 4.f)
						[
							FilterTextBox.ToSharedRef()
						]
						+SVerticalBox::Slot()
							.AutoHeight()
							.Padding(4.f, 4.f, 4.f, 4.f)
							[
								SNew(SBox)
								.HeightOverride(TreeViewHeight)
								.WidthOverride(TreeViewWidth)
								[
									TypeTreeView.ToSharedRef()
								]
							]
						+SVerticalBox::Slot()
						.AutoHeight()
						.Padding(8.f, 0.f, 8.f, 4.f)
						[
							SNew(SBox)
							.Visibility(CustomFilterOptionsWidgets.Num() > 0 ? EVisibility::Visible : EVisibility::Collapsed)
							[
								SAssignNew(CustomWidgetContainer, SHorizontalBox)
								+SHorizontalBox::Slot()
								.VAlign(VAlign_Center)
								.FillWidth(1.f)
								[
									SNew(STextBlock)
									.Text(this, &SPinTypeSelector::GetPinTypeItemCountText)
								]
							]
						]
					]
				]
			];

		static auto FindTreeItemForSubCategoryObject = [](const TArray<FPinTypeTreeItem>& TreeItems, const UObject* SubCategoryObject) -> FPinTypeTreeItem
		{
			if (!SubCategoryObject)
			{
				return nullptr;
			}

			for (const FPinTypeTreeItem& TreeItemCategory : TreeItems)
			{
				if (TreeItemCategory->GetPinType(false).PinCategory == UEdGraphSchema_K2::AllObjectTypes)
				{
					for (const FPinTypeTreeItem& TreeItem : TreeItemCategory->Children)
					{
						// Don't force load subcategory object, since the current pin's class
						// should already have been loaded if we get to this point anyway
						const bool bForceLoadedSubCategoryObject = false;
						if (TreeItem->GetPinType(bForceLoadedSubCategoryObject).PinSubCategoryObject == SubCategoryObject)
						{
							return TreeItem;
						}
					}

					// Objects should only be under the AllObjectTypes category, so once we've found that no need to keep looking
					break;
				}
			}

			return nullptr;
		};

		// If this pin type is an object type, then try to find its tree item (i.e. one with the same SubCategoryObject),
		// since we need a tree item to know which available reference types it has, to then build the list from
		const UObject* CurrentSubObject = bForSecondaryType ? TargetPinType.Get().PinValueType.TerminalSubCategoryObject.Get() : TargetPinType.Get().PinSubCategoryObject.Get();
		FPinTypeTreeItem CurrentPinInfo = FindTreeItemForSubCategoryObject(TypeTreeRoot, CurrentSubObject);

		// Note: Don't allow changing the reference type from compact selectors if it's a map type,
		// since it's a bit ambiguous whether you're changing the key or value type
		if (CurrentPinInfo && !(SelectorType == ESelectorType::Compact && CurrentPinInfo->GetPinType(false).IsMap()))
		{
			GenerateAllowedObjectTypesList(CurrentPinAllowedObjectReferenceTypes, CurrentPinInfo, bForSecondaryType);
			if (CurrentPinAllowedObjectReferenceTypes.Num() > 0)
			{
				TSharedRef<SListView<FObjectReferenceListItem>> ListView = SNew(SListView<FObjectReferenceListItem>)
					.ListItemsSource(&CurrentPinAllowedObjectReferenceTypes)
					.SelectionMode(ESelectionMode::Single)
					.OnGenerateRow(this, &SPinTypeSelector::GenerateObjectReferenceTreeRow)
					.OnSelectionChanged(this, &SPinTypeSelector::OnObjectReferenceSelectionChanged, bForSecondaryType);

				TreeWrapper->AddSlot()
				.AutoHeight()
				[
					SNew(SVerticalBox)
					.Visibility_Lambda([this](){ return SearchText.IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed; })
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(6.f, 4.f, 4.f, 2.f)
					[
						SNew(STextBlock)
							.Text(LOCTEXT("PinTypeSelector_ChangeReferenceTypeHeading", "Change to:"))
							.Font(FAppStyle::GetFontStyle(TEXT("Kismet.TypePicker.CategoryFont")))
							.ColorAndOpacity(FSlateColor::UseForeground())
					]
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(4.f, 0.f, 4.f, 4.f)
					[
						// This is the same setup/styling as GetAllowedObjectTypes()
						SNew(SBorder)
						.BorderImage(FAppStyle::GetBrush("Menu.Background"))
						[
							SNew(SObjectReferenceWidget, PinTypeSelectorMenuOwner)
							[
								SNew(SListViewSelectorDropdownMenu<FObjectReferenceListItem>, nullptr, ListView)
								[
									ListView
								]
							]
						]
					]
				];
			}
		}

		if(CustomWidgetContainer.IsValid())
		{
			for(const TSharedPtr<SWidget>& CustomFilterOptionsWidget : CustomFilterOptionsWidgets)
			{
				CustomWidgetContainer->AddSlot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					CustomFilterOptionsWidget.ToSharedRef()
				];
			}
		}

		if (bForSecondaryType)
		{
			if (SecondaryTypeComboButton.IsValid())
			{
				SecondaryTypeComboButton->SetMenuContentWidgetToFocus(FilterTextBox);
			}
		}
		else
		{
			TypeComboButton->SetMenuContentWidgetToFocus(FilterTextBox);
		}
	}
	else
	{
		// Clear the selection in such a way as to also clear the keyboard selector
		TypeTreeView->SetSelection(NULL, ESelectInfo::OnNavigation);
		TypeTreeView->ClearExpandedItems();
	}

	// Clear the filter text box with each opening
	if( FilterTextBox.IsValid() )
	{
		FilterTextBox->SetText( FText::GetEmpty() );
	}

	return MenuContent.ToSharedRef();
}

TSharedRef<SWidget> SPinTypeSelector::GetPinContainerTypeMenuContent()
{
	FMenuBuilder MenuBuilder(true, nullptr, nullptr, /*bCloseSelfOnly*/true);

	PinTypeSelectorStatics::PinTypes.Reset();
	
	PinTypeSelectorStatics::PinTypes.Add(MakeShared<EPinContainerType>(EPinContainerType::None));
	PinTypeSelectorStatics::PinTypes.Add(MakeShared<EPinContainerType>(EPinContainerType::Array));
	PinTypeSelectorStatics::PinTypes.Add(MakeShared<EPinContainerType>(EPinContainerType::Set));
	PinTypeSelectorStatics::PinTypes.Add(MakeShared<EPinContainerType>(EPinContainerType::Map));


	for (int32 i = PinTypeSelectorStatics::PinTypes.Num()-1; i >= 0; --i)
	{
		if (!Schema->SupportsPinTypeContainer(SchemaAction, TargetPinType.Get(), *PinTypeSelectorStatics::PinTypes[i].Get()))
		{
			PinTypeSelectorStatics::PinTypes.RemoveAt(i);
		}
	}	

	for (auto& PinType : PinTypeSelectorStatics::PinTypes)
	{
		EPinContainerType PinContainerType = *PinType;
		FUIAction Action;
		Action.ExecuteAction.BindSP(this, &SPinTypeSelector::OnContainerTypeSelectionChanged, PinContainerType);
		Action.CanExecuteAction.BindLambda([PinContainerType, this]() { return !ContainerRequiresGetTypeHash(PinContainerType) || FBlueprintEditorUtils::HasGetTypeHash(this->TargetPinType.Get()); });

		const FSlateBrush* SecondaryIcon = PinContainerType == EPinContainerType::Map ? FAppStyle::Get().GetBrush(TEXT("Kismet.VariableList.MapValueTypeIcon")) : nullptr;

		TSharedRef<SWidget> Widget =
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SLayeredImage, SecondaryIcon, TAttribute<FSlateColor>(this, &SPinTypeSelector::GetSecondaryTypeIconColor))
				.Image(FAppStyle::Get().GetBrush(PinTypeSelectorStatics::Images[(int32)PinContainerType]))
				.ToolTip(IDocumentation::Get()->CreateToolTip(PinTypeSelectorStatics::Tooltips[(int32)PinContainerType], nullptr, *PinTypeSelectorStatics::BigTooltipDocLink, TEXT("Containers")))
				.ColorAndOpacity(this, &SPinTypeSelector::GetTypeIconColor)
			]
			+ SHorizontalBox::Slot()
			.Padding(4.0f, 2.0f)
			[
				SNew(STextBlock)
				.Text(PinTypeSelectorStatics::Labels[(int32)PinContainerType])
			];

		MenuBuilder.AddMenuEntry(Action, Widget);
	}

	return MenuBuilder.MakeWidget();
}

template<typename T>
TOptional<T> FindBestMatch(FStringView SearchText, TArrayView<T> Container, TFunctionRef<FString(const T& ContainerItem)> ContainerElementTextExtractor)
{
	if (SearchText.IsEmpty())
	{
		return TOptional<T>();
	}

	if (Container.Num() == 0)
	{
		return TOptional<T>();
	}

	// Multiple candidates, pick one that is closest match.
	auto CalculateScore = [](FStringView SearchValue, FStringView CandidateValue)
	{
		if (CandidateValue.IsEmpty())
		{
			return 0.0f;
		}
		const float WorstCase = static_cast<float>(SearchValue.Len() + CandidateValue.Len());
		return 1.0f - (Algo::LevenshteinDistance(SearchValue, CandidateValue) / WorstCase);
	};

	const FString ObjectNameLowerCase = FString(SearchText).ToLower();

	int32 HighestScoreIndex = 0;
	float HighestScore = CalculateScore(ObjectNameLowerCase, ContainerElementTextExtractor(Container[0]).ToLower());

	for (int32 Index = 1; Index < Container.Num(); Index++)
	{
		const float Score = CalculateScore(ObjectNameLowerCase, ContainerElementTextExtractor(Container[Index]).ToLower());
		if (Score > HighestScore)
		{
			HighestScore = Score;
			HighestScoreIndex = Index;
		}
	}

	return Container[HighestScoreIndex];
}

//=======================================================================
// Search Support
void SPinTypeSelector::OnFilterTextChanged(const FText& NewText)
{
	SearchText = NewText;
	NumFilteredPinTypeItems = NewText.IsEmpty() ? NumValidPinTypeItems : 0;
	FilteredTypeTreeRoot.Reset();

	FTopLevenshteinResult<FPinTypeTreeItem> TopLevenshteinResult;
	GetChildrenMatchingSearch(NewText, TypeTreeRoot, FilteredTypeTreeRoot, TopLevenshteinResult);
	TypeTreeView->RequestTreeRefresh();
	
	if (TopLevenshteinResult.IsSet())
	{
		TypeTreeView->SetSelection(TopLevenshteinResult.Item, ESelectInfo::OnNavigation);
		TypeTreeView->RequestScrollIntoView(TopLevenshteinResult.Item);
	}
}

void SPinTypeSelector::OnFilterTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnEnter)
	{
		auto SelectedItems = TypeTreeView->GetSelectedItems();
		if (SelectedItems.Num() > 0)
		{
			TypeTreeView->SetSelection(SelectedItems[0]);
		}
	}
}

// helpers for dealing with the UEdGraphSchema_K2::FPinTypeTreeInfo tree:
namespace PinTypeSelectorImpl
{
	static void FilterUnsupportedTypesImpl(TArray<FPinTypeTreeItem>& ToFilter, const UEdGraphSchema* Schema, TWeakPtr<const FEdGraphSchemaAction> SchemaAction, const TArray<TSharedPtr<class IPinTypeSelectorFilter>>& CustomFilters);
	static int32 CountValidTreeItemsImpl(const TArray<FPinTypeTreeItem>& ToCount);
}

static void PinTypeSelectorImpl::FilterUnsupportedTypesImpl(TArray<FPinTypeTreeItem>& ToFilter, const UEdGraphSchema* Schema, TWeakPtr<const FEdGraphSchemaAction> SchemaAction, const TArray<TSharedPtr<class IPinTypeSelectorFilter>>& CustomFilters)
{
	const auto FilterFunction = [Schema, SchemaAction, &CustomFilters = CustomFilters](const FPinTypeTreeItem& Item)
	{
		const auto DoCustomFilter = [&CustomFilters = CustomFilters](const FPinTypeTreeItem& Item)
		{
			bool bCustomFilterMatches = true;
			for (const TSharedPtr<IPinTypeSelectorFilter>& CustomFilter : CustomFilters)
			{
				if (CustomFilter.IsValid())
				{
					bCustomFilterMatches &= CustomFilter->ShouldShowPinTypeTreeItem(Item);
				}
			}
			return bCustomFilterMatches;
		};

		const bool bSupportsType = (Item->Children.Num() != 0 ||
			(Schema->SupportsPinType(SchemaAction, Item->GetPinTypeNoResolve()) &&
			 DoCustomFilter(Item)));
		return !bSupportsType;
	};

	for (const FPinTypeTreeItem& Item : ToFilter)
	{
		if (Item->Children.Num() > 0)
		{
			FilterUnsupportedTypesImpl(Item->Children, Schema, SchemaAction, CustomFilters);
		}
	}

	ToFilter.SetNum(Algo::StableRemoveIf(ToFilter, FilterFunction));
}

static int32 PinTypeSelectorImpl::CountValidTreeItemsImpl(const TArray<FPinTypeTreeItem>& ToCount)
{
	int32 Count = Algo::CountIf(ToCount, [](const FPinTypeTreeItem& Item) { return !Item->bReadOnly; });
	for (const FPinTypeTreeItem& Item : ToCount)
	{
		Count += CountValidTreeItemsImpl(Item->Children);
	}
	return Count;
}

void SPinTypeSelector::FilterUnsupportedTypes(TArray<FPinTypeTreeItem>& ToFilter)
{
	// Remove read-only root items if they have no children; there will be no subtree to select non read-only items from in that case
	int32 RootItemIndex = 0;
	while (RootItemIndex < TypeTreeRoot.Num())
	{
		FPinTypeTreeItem TypeTreeItemPtr = TypeTreeRoot[RootItemIndex];
		if (TypeTreeItemPtr.IsValid()
			&& TypeTreeItemPtr->bReadOnly
			&& TypeTreeItemPtr->Children.Num() == 0)
		{
			TypeTreeRoot.RemoveAt(RootItemIndex);
		}
		else
		{
			++RootItemIndex;
		}
	}

	PinTypeSelectorImpl::FilterUnsupportedTypesImpl(ToFilter, Schema, SchemaAction, CustomFilters);

	// count items after filtering invalid ones, set up the 'filtered view' to default to all the valid items:
	NumValidPinTypeItems = PinTypeSelectorImpl::CountValidTreeItemsImpl(ToFilter);
	NumFilteredPinTypeItems = NumValidPinTypeItems;
	FilteredTypeTreeRoot = TypeTreeRoot;
}

bool SPinTypeSelector::GetChildrenMatchingSearch(const FText& InSearchText, const TArray<FPinTypeTreeItem>& UnfilteredList, TArray<FPinTypeTreeItem>& OutFilteredList, FTopLevenshteinResult<FPinTypeTreeItem>& OutTopLevenshteinResult)
{
	const bool bIsEmptySearch = InSearchText.IsEmpty();
	if (bIsEmptySearch)
	{
		OutFilteredList = UnfilteredList;
		return true;
	}

	FString TrimmedFilterString;
	TArray<FString> FilterTerms;
	TArray<FString> SanitizedFilterTerms;

	// Trim and sanitized the filter text (so that it more likely matches the action descriptions)
	TrimmedFilterString = FText::TrimPrecedingAndTrailing(InSearchText).ToString();

	// Tokenize the search box text into a set of terms; all of them must be present to pass the filter
	TrimmedFilterString.ParseIntoArray(FilterTerms, TEXT(" "), true);

	// Generate a list of sanitized versions of the strings
	for (int32 iFilters = 0; iFilters < FilterTerms.Num(); iFilters++)
	{
		FString EachString = FName::NameToDisplayString(FilterTerms[iFilters], false);
		EachString = EachString.Replace(TEXT(" "), TEXT(""));
		SanitizedFilterTerms.Add(EachString);
	}

	// Both of these should match!
	ensure(SanitizedFilterTerms.Num() == FilterTerms.Num());

	bool bReturnVal = false;

	for( const FPinTypeTreeItem& Item : UnfilteredList)
	{
		TArray<FPinTypeTreeItem> ValidChildren;
		const bool bHasChildrenMatchingSearch = GetChildrenMatchingSearch(InSearchText, Item->Children, ValidChildren, OutTopLevenshteinResult);
		bool bFilterMatches = true;

		// If children match the search filter, there's no need to do any additional checks
		if (!bHasChildrenMatchingSearch)
		{
			// but if they don't, we need to match the terms:
			const FString& LocalizedDescriptionString = Item->GetCachedDescriptionString();
			const FString* SourceDescriptionStringPtr = FTextInspector::GetSourceString(Item->GetDescription());

			// Test both the localized and source strings for a match
			const FString MangledLocalizedDescriptionString = LocalizedDescriptionString.Replace(TEXT(" "), TEXT(""));
			const FString MangledSourceDescriptionString = (SourceDescriptionStringPtr && *SourceDescriptionStringPtr != LocalizedDescriptionString) ? SourceDescriptionStringPtr->Replace(TEXT(" "), TEXT("")) : FString();

			for (int32 FilterIndex = 0; FilterIndex < FilterTerms.Num() && bFilterMatches; ++FilterIndex)
			{
				const bool bMatchesLocalizedTerm = MangledLocalizedDescriptionString.Contains(FilterTerms[FilterIndex]) || MangledLocalizedDescriptionString.Contains(SanitizedFilterTerms[FilterIndex]);
				const bool bMatchesSourceTerm = !MangledSourceDescriptionString.IsEmpty() && (MangledSourceDescriptionString.Contains(FilterTerms[FilterIndex]) || MangledSourceDescriptionString.Contains(SanitizedFilterTerms[FilterIndex]));
				bFilterMatches &= (bMatchesLocalizedTerm || bMatchesSourceTerm);
			}
		}

		if(bFilterMatches)
		{
			FPinTypeTreeItem NewInfo = MakeShared<UEdGraphSchema_K2::FPinTypeTreeInfo>(Item);
			NewInfo->Children = MoveTemp(ValidChildren);
			OutFilteredList.Add(NewInfo);

			// Skip anything with children, those are categories and we don't care about those.
			if (Item->Children.Num() == 0)
			{
				OutTopLevenshteinResult.CompareAndUpdate(TrimmedFilterString, NewInfo, Item->GetCachedDescriptionString());
			}

			if (TypeTreeView.IsValid())
			{
				TypeTreeView->SetItemExpansion(NewInfo, !bIsEmptySearch);
			}

			if (!NewInfo->bReadOnly)
			{
				++NumFilteredPinTypeItems;
			}

			bReturnVal = true;
		}
	}

	return bReturnVal;
}

FText SPinTypeSelector::GetToolTipForComboBoxType() const
{
	FText EditText;
	if(IsEnabled())
	{
		if (IsRightClickArrayStateToggleEnabled(SelectorType))
		{
			EditText = LOCTEXT("CompactPinTypeSelector", "Left click to select the variable's pin type. Right click to toggle the type as an array.\n");
		}
		else if (SelectorType == ESelectorType::Full)
		{
			EditText = LOCTEXT("PinTypeSelector", "Select the variable's pin type.\n");
		}
	}
	else
	{
		EditText = LOCTEXT("PinTypeSelector_Disabled", "Cannot edit variable type when they are inherited from parent.\n");
	}

	// With the full selector type, just display the primary type in the tooltip because the secondary type has its own combo box.
	// With the partial selector type, we need to jam everything into the tooltip for the single combo box.
	// Additionally, with the partial selector type, GetCombinedTypeDescription automatically appends "type" to its description.
	if (SelectorType == ESelectorType::Full)
	{
		return FText::Format(LOCTEXT("PrimaryTypeTwoLines", "{0}Current: {1} type"), EditText, GetTypeDescription(true));
	}
	else
	{
		return FText::Format(LOCTEXT("PrimaryCombinedTypeTwoLines", "{0}Current: {1}"), EditText, GetCombinedTypeDescription(true));
	}
}

FText SPinTypeSelector::GetToolTipForComboBoxSecondaryType() const
{
	FText EditText;
	if (IsEnabled())
	{
		EditText = LOCTEXT("PinTypeValueSelector", "Select the map's value type.");
	}
	else
	{
		EditText = LOCTEXT("PinTypeSelector_ValueDisabled", "Cannot edit map value type when they are inherited from parent.");
	}

	return FText::Format(LOCTEXT("SecondaryTypeTwoLines", "{0}\nValue Type: {1}"), EditText, GetSecondaryTypeDescription(true));
}

FText SPinTypeSelector::GetToolTipForArrayWidget() const
{
	if(IsEnabled())
	{
		// The entire widget may be enabled, but the array button disabled because it is an "exec" pin.
		if(TargetPinType.Get().PinCategory == UEdGraphSchema_K2::PC_Exec)
		{
			return LOCTEXT("ArrayCheckBox_ExecDisabled", "Exec pins cannot be arrays.");
		}
		return LOCTEXT("ArrayCheckBox", "Make this variable an array of selected type.");
	}

	return LOCTEXT("ArrayCheckBox_Disabled", "Cannot edit variable type while the variable is placed in a graph or inherited from parent.");
}

FText SPinTypeSelector::GetToolTipForContainerWidget() const
{
	if (TargetPinType.Get().PinCategory == UEdGraphSchema_K2::PC_Exec)
	{
		// The entire widget may be enabled, but the container type button may be disabled because it is an "exec" pin.
		return LOCTEXT("ContainerType_ExecDisabled", "Exec pins cannot be containers.");
	}
	else
	{
		FText EditText;
		if (IsEnabled())
		{
			EditText = LOCTEXT("ContainerType", "Make this variable a container (array, set, or map) of selected type.");
		}
		else
		{
			EditText = LOCTEXT("ContainerType_Disabled", "Cannot edit variable type while the variable is placed in a graph or inherited from parent.");
		}

		FText ContainerTypeText;
		switch (TargetPinType.Get().ContainerType)
		{
		case EPinContainerType::Array:
			ContainerTypeText = LOCTEXT("ContainerTypeTooltip_Array", "Array");
			break;
		case EPinContainerType::Set:
			ContainerTypeText = LOCTEXT("ContainerTypeTooltip_Set", "Set");
			break;
		case EPinContainerType::Map:
			ContainerTypeText = LOCTEXT("ContainerTypeTooltip_Map", "Map (Dictionary)");
			break;
		}
		return ContainerTypeText.IsEmpty() ? EditText : FText::Format(LOCTEXT("ContainerTypeTwoLines", "{0}\nContainer Type: {1}"), EditText, ContainerTypeText);
	}
}

FText SPinTypeSelector::GetPinTypeItemCountText() const
{
	if (NumFilteredPinTypeItems == 1)
	{
		return FText::Format(LOCTEXT("PinTypeItemCount_Single", "{0} item"), FText::AsNumber(NumFilteredPinTypeItems));
	}
	else
	{
		return FText::Format(LOCTEXT("PinTypeItemCount_Plural", "{0} items"), FText::AsNumber(NumFilteredPinTypeItems));
	}
}

FReply SPinTypeSelector::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (IsRightClickArrayStateToggleEnabled(SelectorType) && MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		bIsRightMousePressed = true;
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SPinTypeSelector::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (IsRightClickArrayStateToggleEnabled(SelectorType) && MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		if (bIsRightMousePressed)
		{
			OnArrayStateToggled();
		}
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SPinTypeSelector::OnMouseLeave( const FPointerEvent& MouseEvent )
{
	SCompoundWidget::OnMouseLeave(MouseEvent);
	bIsRightMousePressed = false;
}

void SPinTypeSelector::OnCustomFilterChanged()
{
	// regenerate clean type treeroot:
	GetPinTypeTree.Execute(TypeTreeRoot, TypeTreeFilter);

	// apply new filters
	FilterUnsupportedTypes(TypeTreeRoot);

	// if any search text is present, apply that to the new FilteredTypeTreeRoot
	if(!SearchText.IsEmpty())
	{
		NumFilteredPinTypeItems = 0;
		FilteredTypeTreeRoot.Reset();

		FTopLevenshteinResult<FPinTypeTreeItem> TopLevenshteinResult;
		GetChildrenMatchingSearch(SearchText, TypeTreeRoot, FilteredTypeTreeRoot, TopLevenshteinResult);
		
		// FilteredTypeTreeRoot now up to date, we can refresh
		if (TypeTreeView.IsValid())
		{
			TypeTreeView->RequestTreeRefresh();
		}

		// and update focus:
		if (TopLevenshteinResult.IsSet())
		{
			TypeTreeView->SetSelection(TopLevenshteinResult.Item, ESelectInfo::OnNavigation);
			TypeTreeView->RequestScrollIntoView(TopLevenshteinResult.Item);
		}
	}
	else if (TypeTreeView.IsValid())
	{
		TypeTreeView->RequestTreeRefresh();
	}
}

#undef LOCTEXT_NAMESPACE
