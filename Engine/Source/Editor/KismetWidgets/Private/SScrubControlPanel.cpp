// Copyright Epic Games, Inc. All Rights Reserved.

#include "SScrubControlPanel.h"

#include "EditorWidgetsModule.h"
#include "HAL/PlatformMath.h"
#include "ITransportControl.h"
#include "Layout/Children.h"
#include "Layout/Clipping.h"
#include "Layout/Margin.h"
#include "Math/Vector2D.h"
#include "Modules/ModuleManager.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Types/SlateEnums.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"


#define LOCTEXT_NAMESPACE "SScrubControlPanel"

void SScrubControlPanel::Construct( const SScrubControlPanel::FArguments& InArgs )
{
	ScrubWidget = NULL;

	IsRealtimeStreamingMode = InArgs._IsRealtimeStreamingMode;
	
	FEditorWidgetsModule& EditorWidgetsModule = FModuleManager::Get().LoadModuleChecked<FEditorWidgetsModule>( "EditorWidgets" );
	
	FTransportControlArgs TransportControlArgs;
	TransportControlArgs.OnForwardPlay = InArgs._OnClickedForwardPlay;
	TransportControlArgs.OnRecord = InArgs._OnClickedRecord;
	TransportControlArgs.OnBackwardPlay = InArgs._OnClickedBackwardPlay;
	TransportControlArgs.OnForwardStep = InArgs._OnClickedForwardStep;
	TransportControlArgs.OnBackwardStep = InArgs._OnClickedBackwardStep;
	TransportControlArgs.OnForwardEnd = InArgs._OnClickedForwardEnd;
	TransportControlArgs.OnBackwardEnd = InArgs._OnClickedBackwardEnd;
	TransportControlArgs.OnToggleLooping = InArgs._OnClickedToggleLoop;
	TransportControlArgs.OnGetLooping = InArgs._OnGetLooping;
	TransportControlArgs.OnGetPlaybackMode = InArgs._OnGetPlaybackMode;
	TransportControlArgs.OnGetRecording = InArgs._OnGetRecording;
	TransportControlArgs.OnTickPlayback = InArgs._OnTickPlayback;
	TransportControlArgs.OnGetRecordButtonVisibility = InArgs._OnGetRecordVisibility;
	TransportControlArgs.WidgetsToCreate = InArgs._TransportControlWidgetsToCreate.Get(TArray<FTransportControlWidget>());

	FTransportControlArgs TransportControlArgsForRealtimeStreamingMode;
	TransportControlArgsForRealtimeStreamingMode.OnForwardPlay = TransportControlArgs.OnForwardPlay;
	TransportControlArgsForRealtimeStreamingMode.OnForwardStep = TransportControlArgs.OnForwardStep;
	TransportControlArgsForRealtimeStreamingMode.OnGetPlaybackMode = TransportControlArgs.OnGetPlaybackMode;
	TransportControlArgsForRealtimeStreamingMode.OnGetRecording = TransportControlArgs.OnGetRecording;
	TransportControlArgsForRealtimeStreamingMode.OnRecord = TransportControlArgs.OnRecord;
	TransportControlArgsForRealtimeStreamingMode.OnGetRecordButtonVisibility = InArgs._OnGetRecordVisibility;

	this->ChildSlot
	.Padding( FMargin( 0.0f, 1.0f) )
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.HAlign(HAlign_Fill) 
		.VAlign(VAlign_Center)
		.FillWidth(1)
		.Padding( FMargin( 0.0f, 0.0f) )
		[
			SNew( SBorder )
			[
				SAssignNew(ScrubWidget, SScrubWidget)
				.Clipping(EWidgetClipping::ClipToBounds)
				.Value(InArgs._Value)
				.NumOfKeys(InArgs._NumOfKeys)
				.DisplayDrag(InArgs._DisplayDrag)
				.bDisplayAnimScrubBarEditing(InArgs._bDisplayAnimScrubBarEditing)
				.SequenceLength(InArgs._SequenceLength)
				.OnValueChanged(InArgs._OnValueChanged)
				.OnBeginSliderMovement(InArgs._OnBeginSliderMovement)
				.OnEndSliderMovement(InArgs._OnEndSliderMovement)
				.ViewInputMin(InArgs._ViewInputMin)
				.ViewInputMax(InArgs._ViewInputMax)
				.OnSetInputViewRange(InArgs._OnSetInputViewRange)
				.OnCropAnimSequence(InArgs._OnCropAnimSequence)
				.OnAddAnimSequence(InArgs._OnAddAnimSequence)
				.OnAppendAnimSequence(InArgs._OnAppendAnimSequence)
				.OnReZeroAnimSequence(InArgs._OnReZeroAnimSequence)
				.bAllowZoom(InArgs._bAllowZoom)
				.DraggableBars(InArgs._DraggableBars)
				.OnBarDrag(InArgs._OnBarDrag)
				.OnBarCommit(InArgs._OnBarCommit)
			]
		]

		// Padding
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			// Padding to make controls line up with the track label widths.
			// note: a more robust way to accomplish this would be nice.
			SNew(SSpacer)
			.Size(FVector2D(16.0f, 16.0f))
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SBorder)
			.Padding(0.0f)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			.Visibility(this, &SScrubControlPanel::GetRealtimeControlVisibility, false)
			[
				EditorWidgetsModule.CreateTransportControl(TransportControlArgs)
			]
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SBorder)
			.Padding(0.0f)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			.Visibility(this, &SScrubControlPanel::GetRealtimeControlVisibility, true)
			[
				EditorWidgetsModule.CreateTransportControl(TransportControlArgsForRealtimeStreamingMode)
			]
		]
	];
}

EVisibility SScrubControlPanel::GetRealtimeControlVisibility(bool bIsControlForRealtimeStreamingMode) const
{
	return (IsRealtimeStreamingMode.Get() == bIsControlForRealtimeStreamingMode)
		? EVisibility::Visible : EVisibility::Collapsed;
}

#undef LOCTEXT_NAMESPACE
