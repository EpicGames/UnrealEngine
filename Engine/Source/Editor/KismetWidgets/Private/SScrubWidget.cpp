// Copyright Epic Games, Inc. All Rights Reserved.

#include "SScrubWidget.h"

#include "Animation/AnimTypes.h"
#include "Containers/UnrealString.h"
#include "Fonts/FontMeasure.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GenericPlatform/GenericPlatformMath.h"
#include "GenericPlatform/ICursor.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Geometry.h"
#include "Layout/PaintGeometry.h"
#include "Layout/WidgetPath.h"
#include "Math/Color.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/CString.h"
#include "Rendering/DrawElements.h"
#include "Rendering/RenderingCommon.h"
#include "Rendering/SlateRenderer.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/WidgetStyle.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"
#include "Widgets/Input/STextEntryPopup.h"

class FSlateRect;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "ScrubWidget"

static const float MinStepLengh=15.f;

int32 SScrubWidget::GetDivider(float InputMinX, float InputMaxX, FVector2D WidgetSize, float SequenceLength, int32 NumFrames)
{
	return GetDivider(InputMinX, InputMaxX, UE::Slate::CastToVector2f(WidgetSize), SequenceLength, NumFrames);
}

/** This function is used by a few random widgets and is mostly arbitrary. It could be moved anywhere. */
int32 SScrubWidget::GetDivider(float InputMinX, float InputMaxX, FVector2f WidgetSize, float SequenceLength, int32 NumFrames)
{
	FTrackScaleInfo TimeScaleInfo(InputMinX, InputMaxX, 0.f, 0.f, WidgetSize);
	const FAnimKeyHelper Helper(SequenceLength, NumFrames);

	float TimePerKey = Helper.TimePerKey();

	float TotalWidgetWidth = static_cast<float>(TimeScaleInfo.WidgetSize.X);
	int32 NumKeys = static_cast<int32>(TimeScaleInfo.ViewInputRange / TimePerKey);
	float KeyWidgetWidth = TotalWidgetWidth / static_cast<float>(NumKeys);
	int32 Divider = 1; 
	if (KeyWidgetWidth > 0)
	{
		Divider = FMath::Max<int32>(static_cast<int32>(50 / KeyWidgetWidth), 1);
	}
	return Divider;
}

void SScrubWidget::Construct( const SScrubWidget::FArguments& InArgs )
{
	ValueAttribute = InArgs._Value;
	OnValueChanged = InArgs._OnValueChanged;
	OnBeginSliderMovement = InArgs._OnBeginSliderMovement;
	OnEndSliderMovement = InArgs._OnEndSliderMovement;

	DistanceDragged = 0.0f;
	NumOfKeys = InArgs._NumOfKeys;
	SequenceLength = InArgs._SequenceLength;
	ViewInputMin = InArgs._ViewInputMin;
	ViewInputMax = InArgs._ViewInputMax;
	OnSetInputViewRange = InArgs._OnSetInputViewRange;
	OnCropAnimSequence = InArgs._OnCropAnimSequence;
	OnAddAnimSequence = InArgs._OnAddAnimSequence;
	OnAppendAnimSequence = InArgs._OnAppendAnimSequence;
	OnReZeroAnimSequence = InArgs._OnReZeroAnimSequence;

	DraggableBars = InArgs._DraggableBars;
	OnBarDrag = InArgs._OnBarDrag;
	OnBarCommit = InArgs._OnBarCommit;
	bDisplayDrag = InArgs._DisplayDrag;
	bDisplayAnimScrubBarEditing = InArgs._bDisplayAnimScrubBarEditing;
	bMouseMovedDuringPanning = false;
	bDragging = false;
	bPanning = false;
	DraggableBarIndex = INDEX_NONE;
	DraggingBar = false;

	bAllowZoom = InArgs._bAllowZoom;
}

int32 SScrubWidget::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	const bool bActiveFeedback = IsHovered() || bDragging;
	const FSlateBrush* BackgroundImage = bActiveFeedback ?
		FAppStyle::GetBrush("SpinBox.Background.Hovered") :
		FAppStyle::GetBrush("SpinBox.Background");

	const FSlateBrush* FillImage = bActiveFeedback ?
		FAppStyle::GetBrush("SpinBox.Fill.Hovered") :
		FAppStyle::GetBrush("SpinBox.Fill");

	const int32 BackgroundLayer = LayerId;

	const FSlateFontInfo SmallLayoutFont = FCoreStyle::GetDefaultFontStyle("Regular", 10);

	const bool bEnabled = ShouldBeEnabled( bParentEnabled );
	const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	const int32 TextLayer = BackgroundLayer + 1;

	const FSlateBrush* StyleInfo = FAppStyle::Get().GetBrush("Brushes.Recessed");
	const double GeomHeight = AllottedGeometry.GetLocalSize().Y;

	const FTrackScaleInfo TimeScaleInfo(ViewInputMin.Get(), ViewInputMax.Get(), 0.f, 0.f, AllottedGeometry.GetLocalSize());

	if ( NumOfKeys.Get() > 0 && SequenceLength.Get() > 0)
	{
		const int32 Divider = SScrubWidget::GetDivider( ViewInputMin.Get(), ViewInputMax.Get(), AllottedGeometry.GetLocalSize(), SequenceLength.Get(), NumOfKeys.Get() );
		const FAnimKeyHelper Helper(SequenceLength.Get(), NumOfKeys.Get());
		const float HalfDivider = Divider/2.f;
		
		const float TimePerKey = Helper.TimePerKey();

		for (float KeyVal = 0; KeyVal < Helper.GetNumKeys(); KeyVal += HalfDivider)
		{
			const float CurValue = KeyVal*TimePerKey;
			const float XPos = TimeScaleInfo.InputToLocalX(CurValue);

			if ( FGenericPlatformMath::Fmod(KeyVal, Divider) == 0.f )
			{
				const FVector2D Offset(XPos, 0.f);
				const FVector2D Size(1, GeomHeight);
				// draw each box with key frame
				FSlateDrawElement::MakeBox(
					OutDrawElements,
					BackgroundLayer,
					AllottedGeometry.ToPaintGeometry(Size, FSlateLayoutTransform(Offset)),
					StyleInfo,
					DrawEffects,
					InWidgetStyle.GetColorAndOpacityTint()
					);

				const int32 FrameNumber = static_cast<int32>(KeyVal);
				const FString FrameString = FString::Printf(TEXT("%d"), (FrameNumber));
				const FVector2D TextOffset(XPos+2.f, 0.f);

				const TSharedRef< FSlateFontMeasure > FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
				const FVector2D TextSize = FontMeasureService->Measure(FrameString, SmallLayoutFont);

				FSlateDrawElement::MakeText(
					OutDrawElements,
					TextLayer, 
					AllottedGeometry.ToPaintGeometry(TextSize, FSlateLayoutTransform(TextOffset)), 
					FrameString, 
					SmallLayoutFont, 
					DrawEffects);
			}
 			else if (HalfDivider > 1.f)
 			{
 				const float Height = static_cast<float>(GeomHeight);
 				const FVector2D Offset(XPos, Height*0.25f);
 				const FVector2D Size(1, Height*0.5f);
 				// draw each box with key frame
 				FSlateDrawElement::MakeBox(
 					OutDrawElements,
 					BackgroundLayer,
					AllottedGeometry.ToPaintGeometry(Size, FSlateLayoutTransform(Offset)),
 					StyleInfo,
 					DrawEffects,
 					InWidgetStyle.GetColorAndOpacityTint()
 					);
 
 			}
		}
	}

	if (SequenceLength.Get() > 0 && bDisplayDrag.Get())
	{
		const int32 ArrowLayer = TextLayer + 1;
		{
			const double XPos = TimeScaleInfo.InputToLocalX(ValueAttribute.Get());
			const double Height = AllottedGeometry.GetLocalSize().Y;
			const FVector2D Offset( XPos - Height*0.25f, 0.f );

			FPaintGeometry MyGeometry =	AllottedGeometry.ToPaintGeometry( FVector2f(Height * 0.5f, Height), FSlateLayoutTransform(Offset) );
			FLinearColor ScrubColor = InWidgetStyle.GetColorAndOpacityTint();
			ScrubColor.A = ScrubColor.A*0.5f;
			ScrubColor.B *= 0.1f;
			ScrubColor.G *= 0.1f;
			FSlateDrawElement::MakeBox( 
				OutDrawElements,
				ArrowLayer,
				MyGeometry,
				StyleInfo,
				DrawEffects,
				ScrubColor
				);
		}

		// Draggable Bars
		if ( DraggableBars.IsBound() )
		{
			for ( const float BarValue : DraggableBars.Get() )
			{
				const float BarXPos = TimeScaleInfo.InputToLocalX(BarValue);
				const FVector2D BarOffset(BarXPos-2.f, 0.f);
				const FVector2D Size(4.f, GeomHeight);

				FLinearColor BarColor = InWidgetStyle.GetColorAndOpacityTint();	
				BarColor.R *= 0.1f;
				BarColor.G *= 0.1f;

				FSlateDrawElement::MakeBox(
					OutDrawElements,
					ArrowLayer+1,
					AllottedGeometry.ToPaintGeometry(Size, FSlateLayoutTransform(BarOffset)),
					StyleInfo,
					DrawEffects,
					BarColor
					);
			}
		}

		return FMath::Max( ArrowLayer, SCompoundWidget::OnPaint( Args, AllottedGeometry, MyCullingRect, OutDrawElements, ArrowLayer, InWidgetStyle, bEnabled ) );
	}

	return SCompoundWidget::OnPaint( Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bEnabled );
}

FReply SScrubWidget::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	bool bHandleLeftMouseButton = MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton;
	bool bHandleRightMouseButton = MouseEvent.GetEffectingButton() == EKeys::RightMouseButton && bAllowZoom;

	bMouseMovedDuringPanning = false;
	if ( bHandleLeftMouseButton )
	{
		if(DraggableBarIndex != INDEX_NONE)
		{
			DraggingBar = true;
		} 
		else
		{
			DistanceDragged = 0;
		}

		// This has prevent throttling on so that viewports continue to run whilst dragging the slider
		return FReply::Handled().CaptureMouse( SharedThis(this) ).PreventThrottling();
	}
	else if ( bHandleRightMouseButton )
	{
		bPanning = true;

		// Always capture mouse if we left or right click on the widget
		return FReply::Handled().CaptureMouse(SharedThis(this));
	}

	return FReply::Unhandled();
}
	
FReply SScrubWidget::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	bool bHandleLeftMouseButton = MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && this->HasMouseCapture();
	bool bHandleRightMouseButton = MouseEvent.GetEffectingButton() == EKeys::RightMouseButton && this->HasMouseCapture() && bAllowZoom;

	if ( bHandleRightMouseButton )
	{
		bPanning = false;

		FTrackScaleInfo TimeScaleInfo(ViewInputMin.Get(), ViewInputMax.Get(), 0.f, 0.f, MyGeometry.GetLocalSize());
		float CursorPosX = static_cast<float>(MyGeometry.AbsoluteToLocal(MouseEvent.GetLastScreenSpacePosition()).X);
		float NewValue = TimeScaleInfo.LocalXToInput(CursorPosX);

		if( !bMouseMovedDuringPanning )
		{
			CreateContextMenu(NewValue, MouseEvent);
		}
		return FReply::Handled().ReleaseMouseCapture();
	}
	else if ( bHandleLeftMouseButton )
	{
		if(DraggingBar)
		{
			DraggingBar = false;

			if(OnBarCommit.IsBound())
			{
				const float CursorPosX = static_cast<float>(MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()).X);
				const FTrackScaleInfo ScaleInfo(ViewInputMin.Get(), ViewInputMax.Get(), 0.f, 0.f, MyGeometry.GetLocalSize());
				const float NewDataPos = FMath::Clamp( ScaleInfo.LocalXToInput(CursorPosX), ViewInputMin.Get(), ViewInputMax.Get() );				
				OnBarCommit.ExecuteIfBound(DraggableBarIndex, NewDataPos);
			}			
		}
		else if( bDragging )
		{
			OnEndSliderMovement.ExecuteIfBound( ValueAttribute.Get() );
		}
		else
		{
			FTrackScaleInfo TimeScaleInfo(ViewInputMin.Get(), ViewInputMax.Get(), 0.f, 0.f, MyGeometry.GetLocalSize());
			float CursorPosX = static_cast<float>(MyGeometry.AbsoluteToLocal(MouseEvent.GetLastScreenSpacePosition()).X);
			float NewValue = static_cast<float>(TimeScaleInfo.LocalXToInput(CursorPosX));

			CommitValue(NewValue, true, false);
		}

		bDragging = false;
		return FReply::Handled().ReleaseMouseCapture();
	}

	return FReply::Unhandled();
}
	
FReply SScrubWidget::OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	// Bar Dragging
	if(DraggingBar)
	{
		// Update bar if we are dragging
		float CursorPosX = static_cast<float>(MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()).X);
		FTrackScaleInfo ScaleInfo(ViewInputMin.Get(), ViewInputMax.Get(), 0.f, 0.f, MyGeometry.GetLocalSize());
		float NewDataPos = FMath::Clamp(ScaleInfo.LocalXToInput(CursorPosX), ViewInputMin.Get(), ViewInputMax.Get());
		OnBarDrag.ExecuteIfBound(DraggableBarIndex, NewDataPos);
	}
	else
	{
		// Update what bar we are hovering over
		float CursorPosX = static_cast<float>(MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()).X);
		FTrackScaleInfo ScaleInfo(ViewInputMin.Get(), ViewInputMax.Get(), 0.f, 0.f, MyGeometry.GetLocalSize());
		DraggableBarIndex = INDEX_NONE;
		if ( DraggableBars.IsBound() )
		{
			const TArray<float>& DraggableBarsVal = DraggableBars.Get();
			for ( int32 I=0; I < DraggableBarsVal.Num(); I++ )
			{
				if(FMath::Abs( ScaleInfo.InputToLocalX(DraggableBarsVal[I]) - CursorPosX ) < 10)
				{
					DraggableBarIndex = I;
					break;
				}
			}
		}
	}

	if ( this->HasMouseCapture() )
	{
		if (MouseEvent.IsMouseButtonDown( EKeys::RightMouseButton ) && bPanning)
		{
			FTrackScaleInfo ScaleInfo(ViewInputMin.Get(), ViewInputMax.Get(), 0.f, 0.f, MyGeometry.GetLocalSize());
			FVector2D ScreenDelta = MouseEvent.GetCursorDelta();
			float InputDeltaX = static_cast<float>(ScreenDelta.X / ScaleInfo.PixelsPerInput);

			bMouseMovedDuringPanning |= !ScreenDelta.IsNearlyZero(0.001f);

			float NewViewInputMin = ViewInputMin.Get() - InputDeltaX;
			float NewViewInputMax = ViewInputMax.Get() - InputDeltaX;
			// we'd like to keep  the range if outside when panning
			if ( NewViewInputMin < 0.f )
			{
				NewViewInputMin = 0.f;
				NewViewInputMax = ScaleInfo.ViewInputRange;
			}
			else if ( NewViewInputMax > SequenceLength.Get() )
			{
				NewViewInputMax = SequenceLength.Get();
				NewViewInputMin = NewViewInputMax - ScaleInfo.ViewInputRange;
			}

			OnSetInputViewRange.ExecuteIfBound(NewViewInputMin, NewViewInputMax);
		}
		else if (!bDragging)
		{
			float CursorDeltaX = static_cast<float>(FMath::Abs(MouseEvent.GetCursorDelta().X));
			DistanceDragged += CursorDeltaX;
			if ( DistanceDragged > FSlateApplication::Get().GetDragTriggerDistance() )
			{
				bDragging = true;
			}
			if( bDragging )
			{
				OnBeginSliderMovement.ExecuteIfBound();
			}
		}
		else if (bDragging)
		{
			FTrackScaleInfo TimeScaleInfo(ViewInputMin.Get(), ViewInputMax.Get(), 0.f, 0.f, MyGeometry.GetLocalSize());
			float CursorPosX = static_cast<float>(MyGeometry.AbsoluteToLocal(MouseEvent.GetLastScreenSpacePosition()).X);
			float NewValue = TimeScaleInfo.LocalXToInput(CursorPosX);

			CommitValue( NewValue, true, false );
		}
		return FReply::Handled();
	}

	

	return FReply::Unhandled();
}

// FCursorReply SScrubWidget::OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const
// {
// 	return FCursorReply::Cursor( EMouseCursor::ResizeLeftRight );
// }


void SScrubWidget::CommitValue( float NewValue, bool bSliderClamp, bool bCommittedFromText )
{
	if ( !ValueAttribute.IsBound() )
	{
		ValueAttribute.Set( NewValue );
	}

	OnValueChanged.ExecuteIfBound( NewValue );
}

FVector2D SScrubWidget::ComputeDesiredSize( float ) const
{
	return FVector2D(100, 30);
}

FReply SScrubWidget::OnMouseWheel( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ( bAllowZoom && OnSetInputViewRange.IsBound() )
	{
		const float ZoomDelta = -0.1f * MouseEvent.GetWheelDelta();

		{
			const float InputViewSize = ViewInputMax.Get() - ViewInputMin.Get();
			const float InputChange = InputViewSize * ZoomDelta;

			float ViewMinInput = ViewInputMin.Get() - (InputChange * 0.5f);
			float ViewMaxInput = ViewInputMax.Get() + (InputChange * 0.5f);

			OnSetInputViewRange.Execute(ViewMinInput, ViewMaxInput);
		}

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FCursorReply SScrubWidget::OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const
{
	if (DraggableBarIndex != INDEX_NONE)
	{
		return FCursorReply::Cursor( EMouseCursor::ResizeLeftRight );
	}

	return FCursorReply::Unhandled();
}

void SScrubWidget::CreateContextMenu(float CurrentFrameTime, const FPointerEvent& MouseEvent)
{
	if (bDisplayAnimScrubBarEditing && 
		(OnCropAnimSequence.IsBound() || OnReZeroAnimSequence.IsBound() || OnAddAnimSequence.IsBound()) && 
		(SequenceLength.Get() >= MINIMUM_ANIMATION_LENGTH))
	{
		const bool CloseAfterSelection = true;
		FMenuBuilder MenuBuilder( CloseAfterSelection, NULL );

		MenuBuilder.BeginSection("SequenceEditingContext", LOCTEXT("SequenceEditing", "Sequence Editing") );
		{
			float CurrentFrameFraction = CurrentFrameTime / SequenceLength.Get();
			int32 CurrentFrameNumber = static_cast<int32>(CurrentFrameFraction * NumOfKeys.Get());

			FUIAction Action;
			FText Label;

			if (OnCropAnimSequence.IsBound())
			{
				//Menu - "Remove Before"
				//Only show this option if the selected frame is greater than frame 1 (first frame)
				if (CurrentFrameNumber > 0)
				{
					CurrentFrameFraction = float(CurrentFrameNumber) / (float)NumOfKeys.Get();

					//Corrected frame time based on selected frame number
					float CorrectedFrameTime = CurrentFrameFraction * SequenceLength.Get();

					Action = FUIAction(FExecuteAction::CreateSP(this, &SScrubWidget::OnSequenceCroppedCalled, true, CorrectedFrameTime));
					Label = FText::Format(LOCTEXT("RemoveTillFrame", "Remove frame 0 to frame {0}"), FText::AsNumber(CurrentFrameNumber));
					MenuBuilder.AddMenuEntry(Label, LOCTEXT("RemoveBefore_ToolTip", "Remove sequence before current position"), FSlateIcon(), Action);
				}

				uint32 NextFrameNumber = CurrentFrameNumber + 1;

				//Menu - "Remove After"
				//Only show this option if next frame (CurrentFrameNumber + 1) is valid
				if (NextFrameNumber < NumOfKeys.Get())
				{
					float NextFrameFraction = float(NextFrameNumber) / (float)NumOfKeys.Get();
					float NextFrameTime = NextFrameFraction * SequenceLength.Get();
					Action = FUIAction(FExecuteAction::CreateSP(this, &SScrubWidget::OnSequenceCroppedCalled, false, NextFrameTime));
					Label = FText::Format(LOCTEXT("RemoveFromFrame", "Remove from frame {0} to frame {1}"), FText::AsNumber(NextFrameNumber), FText::AsNumber(NumOfKeys.Get()));
					MenuBuilder.AddMenuEntry(Label, LOCTEXT("RemoveAfter_ToolTip", "Remove sequence after current position"), FSlateIcon(), Action);
				}
			}

			if (OnAddAnimSequence.IsBound())
			{
				MenuBuilder.AddMenuSeparator();

				//Corrected frame time based on selected frame number
				float CorrectedFrameTime = CurrentFrameFraction * SequenceLength.Get();

				Action = FUIAction(FExecuteAction::CreateSP(this, &SScrubWidget::OnSequenceAddedCalled, true, CurrentFrameNumber));
				Label = FText::Format(LOCTEXT("InsertBeforeCurrentFrame", "Insert frame before {0}"), FText::AsNumber(CurrentFrameNumber));
				MenuBuilder.AddMenuEntry(Label, LOCTEXT("InsertBefore_ToolTip", "Insert a frame before current position"), FSlateIcon(), Action);

				Action = FUIAction(FExecuteAction::CreateSP(this, &SScrubWidget::OnSequenceAddedCalled, false, CurrentFrameNumber));
				Label = FText::Format(LOCTEXT("InsertAfterCurrentFrame", "Insert frame after {0}"), FText::AsNumber(CurrentFrameNumber));
				MenuBuilder.AddMenuEntry(Label, LOCTEXT("InsertAfter_ToolTip", "Insert a frame after current position"), FSlateIcon(), Action);
			}

			if(OnAppendAnimSequence.IsBound())
			{
				MenuBuilder.AddMenuSeparator();

				//Corrected frame time based on selected frame number
				Action = FUIAction(FExecuteAction::CreateSP(this, &SScrubWidget::OnShowPopupOfAppendAnimation, true));
				MenuBuilder.AddMenuEntry(LOCTEXT("AppendBegin", "Append in the beginning"), LOCTEXT("AppendBegin_ToolTip", "Append in the beginning"), FSlateIcon(), Action);

				Action = FUIAction(FExecuteAction::CreateSP(this, &SScrubWidget::OnShowPopupOfAppendAnimation, false));
				MenuBuilder.AddMenuEntry(LOCTEXT("AppendEnd", "Append at the end"), LOCTEXT("AppendEnd_ToolTip", "Append at the end"), FSlateIcon(), Action);
			}

			if (OnReZeroAnimSequence.IsBound())
			{
				MenuBuilder.AddMenuSeparator();
				//Menu - "ReZero"
				Action = FUIAction(FExecuteAction::CreateSP(this, &SScrubWidget::OnReZeroCalled, CurrentFrameNumber));
				Label = FText::Format(LOCTEXT("ReZeroAtFrame", "Re-zero at frame {0}"), FText::AsNumber(CurrentFrameNumber));
				MenuBuilder.AddMenuEntry(Label, FText::Format(LOCTEXT("ReZeroAtFrame_ToolTip", "Resets the root track to (0, 0, 0) at frame {0} and apply the difference to all root transform of the sequence. It moves whole sequence to the amount of current root transform."), FText::AsNumber(CurrentFrameNumber)), FSlateIcon(), Action);

				const int32 FrameNumberForCurrentTime = INDEX_NONE;
				Action = FUIAction(FExecuteAction::CreateSP(this, &SScrubWidget::OnReZeroCalled, FrameNumberForCurrentTime));
				Label = LOCTEXT("ReZeroAtCurrentTime", "Re-zero at current time");
				MenuBuilder.AddMenuEntry(Label, LOCTEXT("ReZeroAtCurrentTime_ToolTip", "Resets the root track to (0, 0, 0) at the animation scrub time and apply the difference to all root transform of the sequence. It moves whole sequence to the amount of current root transform."), FSlateIcon(), Action);
			}
		}
		MenuBuilder.EndSection();

		FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();

		FSlateApplication::Get().PushMenu(SharedThis(this), WidgetPath, MenuBuilder.MakeWidget(), FSlateApplication::Get().GetCursorPos(), FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
	}
}

void SScrubWidget::OnSequenceCroppedCalled( bool bFromStart, float CurrentFrameTime )
{
	OnCropAnimSequence.ExecuteIfBound( bFromStart, CurrentFrameTime );

	//Update scrub widget's min and max view output.
	OnSetInputViewRange.ExecuteIfBound( ViewInputMin.Get(), ViewInputMax.Get() );
}

void SScrubWidget::OnSequenceAddedCalled(bool bBefore, int32 CurrentFrameNumber)
{
	OnAddAnimSequence.ExecuteIfBound(bBefore, CurrentFrameNumber);

	//Update scrubs new length to be new Sequence Length
	// @Todo fixme: this whole thing needs to change to "Refresh" 
	// - including the OnSequenceCroppedCalled
	OnSetInputViewRange.ExecuteIfBound(ViewInputMin.Get(), SequenceLength.Get());
}

void SScrubWidget::OnReZeroCalled(int32 FrameIndex)
{
	OnReZeroAnimSequence.ExecuteIfBound(FrameIndex);
}

void SScrubWidget::OnShowPopupOfAppendAnimation(bool bBegin)
{
	TSharedRef<STextEntryPopup> TextEntry =
		SNew(STextEntryPopup)
		.Label(LOCTEXT("AppendAnim_AskNumFrames", "Number of Frames to Append"))
		.OnTextCommitted(this, &SScrubWidget::OnSequenceAppendedCalled, bBegin);

	// Show dialog to enter new track name
	FSlateApplication::Get().PushMenu(
		SharedThis(this),
		FWidgetPath(),
		TextEntry,
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect(FPopupTransitionEffect::TypeInPopup)
		);
}

void SScrubWidget::OnSequenceAppendedCalled(const FText & InNewGroupText, ETextCommit::Type CommitInfo, bool bBegin)
{
	// just a concern
	const static int32 MaxFrame = 1000;

	// handle only onEnter. This is a big thing to apply when implicit focus change or any other event
	if (CommitInfo == ETextCommit::OnEnter)
	{
		int32 NumFrames = FCString::Atoi(*InNewGroupText.ToString());
		if (NumFrames > 0 && NumFrames < MaxFrame)
		{
			OnAppendAnimSequence.ExecuteIfBound(bBegin, NumFrames);
			FSlateApplication::Get().DismissAllMenus();
		}
	}
}

#undef LOCTEXT_NAMESPACE