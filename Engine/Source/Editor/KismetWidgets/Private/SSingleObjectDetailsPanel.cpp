// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSingleObjectDetailsPanel.h"

#include "DetailsViewArgs.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformCrt.h"
#include "IDetailsView.h"
#include "Layout/Children.h"
#include "Misc/AssertionMacros.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SlotBase.h"
#include "Widgets/SBoxPanel.h"

class SWidget;
class UObject;
struct FGeometry;

/////////////////////////////////////////////////////
// SSingleObjectDetailsPanel

void SSingleObjectDetailsPanel::Construct(const FArguments& InArgs, bool bAutomaticallyObserveViaGetObjectToObserve, bool bAllowSearch)
{
	// Create a property view
	FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = bAllowSearch;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.HostCommandList = InArgs._HostCommandList;
	DetailsViewArgs.HostTabManager = InArgs._HostTabManager;

	PropertyView = EditModule.CreateDetailView(DetailsViewArgs);
	
	bAutoObserveObject = bAutomaticallyObserveViaGetObjectToObserve;

	// Create the border that all of the content will get stuffed into
	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding( 3.0f, 2.0f )
		[
			PopulateSlot(PropertyView.ToSharedRef())
		]
	];
}

UObject* SSingleObjectDetailsPanel::GetObjectToObserve() const
{
	return NULL;
}

void SSingleObjectDetailsPanel::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	if (bAutoObserveObject)
	{
		UObject* CurrentObject = GetObjectToObserve();
		if (LastObservedObject.Get() != CurrentObject)
		{
			LastObservedObject = CurrentObject;

			TArray<UObject*> SelectedObjects;
			if (CurrentObject != NULL)
			{
				SelectedObjects.Add(CurrentObject);
			}

			SetPropertyWindowContents(SelectedObjects);
		}
	}
}

void SSingleObjectDetailsPanel::SetPropertyWindowContents(TArray<UObject*> Objects)
{
	if (FSlateApplication::IsInitialized())
	{
		check(PropertyView.IsValid());
		PropertyView->SetObjects(Objects);
	}
}

TSharedRef<SWidget> SSingleObjectDetailsPanel::PopulateSlot(TSharedRef<SWidget> PropertyEditorWidget)
{
	return PropertyEditorWidget;
}
