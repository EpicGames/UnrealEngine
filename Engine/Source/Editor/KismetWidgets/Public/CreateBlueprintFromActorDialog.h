// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "GameFramework/Actor.h"
#include "Misc/EnumClassFlags.h"

class AActor;
class UClass;

//////////////////////////////////////////////////////////////////////////
// FCreateBlueprintFromActorDialog

/** Enum defining which mode to use when creating the blueprint from the selected actors */
enum class ECreateBlueprintFromActorMode : uint8
{
	None,             // Indicates that a user is unable to create blueprints from the currently selected actor set
	Harvest = 1,      // Harvest all the components of the selected actors and create an actor blueprint with those components in it.
	Subclass = 2,     // Create a subclass of the selected Actor's class with defaults from the selected Actor. Valid only when there is a single selected actor.
	ChildActor = 4    // Create an actor blueprint with each selected actor as a child actor component
};
ENUM_CLASS_FLAGS(ECreateBlueprintFromActorMode)

class FCreateBlueprintFromActorDialog
{
public:

	/** 
	 * Static function to access constructing this window.
	 *
	 * @param CreateMode		The mode to use when creating a blueprint from the selected actors
	 * @param ActorOverride		If set convert the specified actor, if null use the currently selected actor
	 */
	static KISMETWIDGETS_API void OpenDialog(ECreateBlueprintFromActorMode CreateMode, AActor* InActorOverride = nullptr, bool bInReplaceActors = true);

	/** 
	 * Static function that returns which create modes are valid given the current selection set
	 */
	static KISMETWIDGETS_API ECreateBlueprintFromActorMode GetValidCreationMethods();

private:

	/** 
	 * Create the blueprint in response to the path being specified by the user
	 *
	 * @param InAssetPath		Path of the asset to create
	 * @param bInHarvest		The mode to use when creating a blueprint from the selected actors
	 */
	static void OnCreateBlueprint(const FString& InAssetPath, UClass* ParentClass, ECreateBlueprintFromActorMode CreateMode, AActor* ActorToUse, bool bInReplaceActors);
};
