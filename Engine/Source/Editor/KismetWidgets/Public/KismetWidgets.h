// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

/**
 * Slate style module, loaded by SlateApplication dynamically at startup
 */
class FKismetWidgetsModule : public IModuleInterface
{

public:
	virtual void StartupModule() override
	{
		
	}

	virtual void ShutdownModule() override
	{
		
	}
};
