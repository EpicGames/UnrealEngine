// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SPinTypeSelector.h"

#include "PinTypeSelectorFilter.generated.h"

UCLASS(abstract, transient, MinimalAPI, config=Editor)
class UPinTypeSelectorFilter : public UObject
{
	GENERATED_BODY()
public:
	UPROPERTY(config)
	TSoftClassPtr<UPinTypeSelectorFilter> FilterClass = nullptr;

	virtual TSharedPtr<IPinTypeSelectorFilter> GetPinTypeSelectorFilter() const { return TSharedPtr<IPinTypeSelectorFilter>(); }
};