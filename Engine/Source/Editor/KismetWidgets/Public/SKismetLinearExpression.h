// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"

#define UE_API KISMETWIDGETS_API

class SWidget;
class UEdGraphNode;
class UEdGraphPin;

//////////////////////////////////////////////////////////////////////////
// SKismetLinearExpression

class SKismetLinearExpression : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SKismetLinearExpression )
		: _IsEditable(true)
		{}

		SLATE_ATTRIBUTE( bool, IsEditable )
	SLATE_END_ARGS()
public:
	UE_API void Construct(const FArguments& InArgs, UEdGraphPin* InitialInputPin);

	UE_API void SetExpressionRoot(UEdGraphPin* InputPin);
protected:
	UE_API TSharedRef<SWidget> MakeNodeWidget(const UEdGraphNode* Node, const UEdGraphPin* FromPin);
	UE_API TSharedRef<SWidget> MakePinWidget(const UEdGraphPin* Pin);

protected:
	TAttribute<bool> IsEditable;
	TSet<const UEdGraphNode*> VisitedNodes;
};

#undef UE_API
