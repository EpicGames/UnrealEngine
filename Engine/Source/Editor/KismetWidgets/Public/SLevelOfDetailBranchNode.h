// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"

#define UE_API KISMETWIDGETS_API

class SWidget;
struct FGeometry;

/////////////////////////////////////////////////////
// SLevelOfDetailBranchNode

DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<SWidget>, FOnGetActiveDetailSlotContent, bool);


template <>
struct TWidgetTypeTraits<class SLevelOfDetailBranchNode>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SLevelOfDetailBranchNode : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SLevelOfDetailBranchNode)
		: _UseLowDetailSlot(false)
		{}

		// Should the low detail or high detail slot be shown?
		SLATE_ATTRIBUTE(bool, UseLowDetailSlot)

		// The low-detail slot
		SLATE_NAMED_SLOT(FArguments, LowDetail)

		// The high-detail slot
		SLATE_NAMED_SLOT(FArguments, HighDetail)

		SLATE_EVENT(FOnGetActiveDetailSlotContent, OnGetActiveDetailSlotContent)
	SLATE_END_ARGS()

public:
	UE_API SLevelOfDetailBranchNode();

	UE_API void Construct(const FArguments& InArgs);

	// SWidget interface
	UE_API virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	// End of SWidget interface

	// Determine whether we need to show the low-detail slot or high-detail slot
	UE_API void RefreshLODSlotContent();

protected:
	// What kind of slot was shown last frame
	int LastCachedValue;

	// The attribute indicating the kind of slot to show
	TAttribute<bool> ShowLowDetailAttr;

	// The low-detail child slot
	TSharedRef<SWidget> ChildSlotLowDetail;

	// The high-detail child slot
	TSharedRef<SWidget> ChildSlotHighDetail;

	FOnGetActiveDetailSlotContent OnGetActiveDetailSlotContent;

};

#undef UE_API
