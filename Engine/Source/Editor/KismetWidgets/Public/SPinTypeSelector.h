// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "SlateFwd.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"
#include "Algo/LevenshteinDistance.h"

#define UE_API KISMETWIDGETS_API

class ITableRow;
class SComboButton;
class SMenuOwner;
class SSearchBox;
class SToolTip;
class SWidget;
class UEdGraphSchema;
struct FEdGraphSchemaAction;
struct FGeometry;
struct FObjectReferenceType;
struct FPointerEvent;
struct FSlateBrush;

DECLARE_DELEGATE_OneParam(FOnPinTypeChanged, const FEdGraphPinType&)

template <typename ItemType>
struct FTopLevenshteinResult
{
	ItemType Item;
	float Score = INDEX_NONE;

	bool IsSet() const { return Score != static_cast<float>(INDEX_NONE); }

	void CompareAndUpdate(FStringView SearchValue, const ItemType& NewItem, FStringView NewItemValue)
	{
		if (SearchValue.IsEmpty() || NewItemValue.IsEmpty())
		{
			return;
		}

		const float WorstCase = static_cast<float>(SearchValue.Len() + NewItemValue.Len());
		const float NormalizedDistance = 1.0f - static_cast<float>(Algo::LevenshteinDistance(SearchValue, NewItemValue)) / WorstCase;
		if (NormalizedDistance > Score)
		{
			Score = NormalizedDistance;
			Item = NewItem;
		}
	}
};


//////////////////////////////////////////////////////////////////////////
// SPinTypeSelector

typedef TSharedPtr<class UEdGraphSchema_K2::FPinTypeTreeInfo> FPinTypeTreeItem;
typedef STreeView<FPinTypeTreeItem> SPinTypeTreeView;

DECLARE_DELEGATE_TwoParams(FGetPinTypeTree, TArray<FPinTypeTreeItem >&, ETypeTreeFilter);

struct FObjectReferenceType;

typedef TSharedPtr<struct FObjectReferenceType> FObjectReferenceListItem;

/** Widget for modifying the type for a variable or pin */
class SPinTypeSelector : public SCompoundWidget
{
public:
	static UE_API TSharedRef<SWidget> ConstructPinTypeImage(const FSlateBrush* PrimaryIcon, const FSlateColor& PrimaryColor, const FSlateBrush* SecondaryIcon, const FSlateColor& SecondaryColor, TSharedPtr<SToolTip> InToolTip);
	static UE_API TSharedRef<SWidget> ConstructPinTypeImage(TAttribute<const FSlateBrush*> PrimaryIcon, TAttribute<FSlateColor> PrimaryColor, TAttribute<const FSlateBrush*> SecondaryIcon, TAttribute<FSlateColor> SecondaryColor );
	static UE_API TSharedRef<SWidget> ConstructPinTypeImage(UEdGraphPin* Pin);

	/** Which type of selector should be used: compact or full mode, or not a selector at all, but just the type image. */
	enum class ESelectorType : uint8
	{
		None,
		// Shows only the type selector pill
		Compact,
		// Shows the type selector pill and full name of type
		Partial,
		// Shows the type selector pill, full name of type, and container selection type
		Full
	};

	SLATE_BEGIN_ARGS(SPinTypeSelector)
		: _TargetPinType()
		, _Schema(nullptr)
		, _SchemaAction(nullptr)
		, _TypeTreeFilter(ETypeTreeFilter::None)
		, _bAllowArrays(true)
		, _TreeViewWidth(300.f)
		, _TreeViewHeight(350.f)
		, _Font(FAppStyle::GetFontStyle(TEXT("NormalFont")))
		, _SelectorType(ESelectorType::Full)
		, _ReadOnly(false)
		{}
		SLATE_ATTRIBUTE( FEdGraphPinType, TargetPinType )
		SLATE_ARGUMENT( const UEdGraphSchema*, Schema )
		SLATE_ARGUMENT( TWeakPtr<const FEdGraphSchemaAction>, SchemaAction)
		SLATE_ARGUMENT( ETypeTreeFilter, TypeTreeFilter )
		SLATE_ARGUMENT( bool, bAllowArrays )
		SLATE_ATTRIBUTE( FOptionalSize, TreeViewWidth )
		SLATE_ATTRIBUTE( FOptionalSize, TreeViewHeight )
		SLATE_EVENT( FOnPinTypeChanged, OnPinTypePreChanged )
		SLATE_EVENT( FOnPinTypeChanged, OnPinTypeChanged )
		SLATE_ATTRIBUTE( FSlateFontInfo, Font )
		SLATE_ARGUMENT( ESelectorType, SelectorType )
		SLATE_ATTRIBUTE(bool, ReadOnly)
		SLATE_ARGUMENT_DEPRECATED(TSharedPtr<class IPinTypeSelectorFilter>, CustomFilter, 5.1, "Please use CustomFilters instead")
		SLATE_ARGUMENT(TArray<TSharedPtr<class IPinTypeSelectorFilter>>, CustomFilters)
	SLATE_END_ARGS()
public:
	UE_API void Construct(const FArguments& InArgs, FGetPinTypeTree GetPinTypeTreeFunc);

	// SWidget interface
	UE_API virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	UE_API virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	UE_API virtual void OnMouseLeave( const FPointerEvent& MouseEvent ) override;
	// End of SWidget interface

protected:
	/** Gets the icon (value, array, set, or map) for the type being manipulated */
	UE_API const FSlateBrush* GetTypeIconImage() const;

	/** Gets the secondary icon (for maps, otherwise null) for the type being manipulated */
	UE_API const FSlateBrush* GetSecondaryTypeIconImage() const;

	/** Gets the type-specific color for the type being manipulated */
	UE_API FSlateColor GetTypeIconColor() const;

	/** Gets the secondary type-specific color for the type being manipulated */
	UE_API FSlateColor GetSecondaryTypeIconColor() const;

	/** Gets a succinct type description for the type being manipulated */
	UE_API FText GetTypeDescription(bool bIncludeSubcategory = false) const;

	/** Gets the secondary type description. E.g. the value type for TMaps */
	UE_API FText GetSecondaryTypeDescription(bool bIncludeSubcategory = false) const;

	/** Gets a combined description of the primary, container, and secondary types. E.g. "Map of Strings to Floats" */
	UE_API FText GetCombinedTypeDescription(bool bIncludeSubcategory = false) const;

	TSharedPtr<SComboButton>		TypeComboButton;
	TSharedPtr<SComboButton>		SecondaryTypeComboButton;
	TSharedPtr<SSearchBox>			FilterTextBox;
	TSharedPtr<SPinTypeTreeView>	TypeTreeView;
	
	/** The pin attribute that we're modifying with this widget */
	TAttribute<FEdGraphPinType>		TargetPinType;

	/** Delegate that is called every time the pin type changes (before and after). */
	FOnPinTypeChanged			OnTypeChanged;
	FOnPinTypeChanged			OnTypePreChanged;

	/** Delegate for the type selector to retrieve the pin type tree (passed into the Construct so the tree can depend on the situation) */
	FGetPinTypeTree				GetPinTypeTree;

	/** Schema in charge of determining available types for this pin */
	const UEdGraphSchema*				Schema;

	/** Schema action related to the pin selection */
	TWeakPtr<const FEdGraphSchemaAction> SchemaAction;

	/** UEdgraphSchema::ETypeTreeFilter flags for filtering available types*/
	ETypeTreeFilter				TypeTreeFilter;

	/** Desired width of the tree view widget */
	TAttribute<FOptionalSize>	TreeViewWidth;

	/** Desired height of the tree view widget */
	TAttribute<FOptionalSize>	TreeViewHeight;

	/** TRUE when the right mouse button is pressed, keeps from handling a right click that does not begin in the widget */
	bool bIsRightMousePressed;

	/** true if GetMenuContent was last called with bForSecondaryType == true */
	bool bMenuContentIsSecondary = false;

	/** Whether the selector is using the compact or full mode, or not a selector at all, but just the type image.*/
	ESelectorType SelectorType;

	/** Whether or not the type is read only and not editable (implies a different style) */
	TAttribute<bool> ReadOnly;

	/** Total number of filtered pin type items. This count excludes category items and reference subtypes. */
	int32 NumFilteredPinTypeItems;

	/** Total number of valid pin type items, that could be filtered. This count excludes category items and reference subtypes. */
	int32 NumValidPinTypeItems;

	/** Holds a cache of the allowed Object Reference types for the last sub-menu opened. */
	TArray<FObjectReferenceListItem> AllowedObjectReferenceTypes;
	TWeakPtr<SListView<FObjectReferenceListItem>> WeakListView;
	TWeakPtr<SMenuOwner> PinTypeSelectorMenuOwner;

	/** Holds a cache of the allowed Object Reference types for the current pin type, to be shown inline. */
	TArray<FObjectReferenceListItem> CurrentPinAllowedObjectReferenceTypes;

	/** An interface to optionally apply a custom filter to the available pin type items for display. */
	TArray<TSharedPtr<class IPinTypeSelectorFilter>> CustomFilters;

	/** Array checkbox support functions */
	UE_API ECheckBoxState IsArrayChecked() const;
	UE_API void OnArrayCheckStateChanged(ECheckBoxState NewState);

	/** Toggles the variable type as an array */
	UE_API void OnArrayStateToggled();

	/** Updates the variable container type: */
	UE_API void OnContainerTypeSelectionChanged(EPinContainerType PinContainerType);

	/** Array containing the unfiltered list of all supported types this pin could possibly have */
	TArray<FPinTypeTreeItem>		TypeTreeRoot;
	/** Array containing a filtered list, according to the text in the searchbox */
	TArray<FPinTypeTreeItem>		FilteredTypeTreeRoot;

	/** Treeview support functions */
	UE_API virtual TSharedRef<ITableRow> GenerateTypeTreeRow(FPinTypeTreeItem InItem, const TSharedRef<STableViewBase>& OwnerTree, bool bForSecondaryType);
	UE_API void OnTypeSelectionChanged(FPinTypeTreeItem Selection, ESelectInfo::Type SelectInfo, bool bForSecondaryType);
	UE_API void GetTypeChildren(FPinTypeTreeItem InItem, TArray<FPinTypeTreeItem>& OutChildren);

	/** Listview support functions for sub-menu */
	UE_API TSharedRef<ITableRow> GenerateObjectReferenceTreeRow(FObjectReferenceListItem InItem, const TSharedRef<STableViewBase>& OwnerTree);
	UE_API void OnObjectReferenceSelectionChanged(FObjectReferenceListItem InItem, ESelectInfo::Type SelectInfo, bool bForSecondaryType);

	/** Reference to the menu content that's displayed when the type button is clicked on */
	TSharedPtr<SMenuOwner> MenuContent;
	FEdGraphPinType MenuContextPinType;
	 
	UE_API virtual TSharedRef<SWidget>	GetMenuContent(bool bForSecondaryType);
	UE_API TSharedRef<SWidget> GetPinContainerTypeMenuContent();

	/** Type searching support */
	FText SearchText;
	UE_API void OnFilterTextChanged(const FText& NewText);
	UE_API void OnFilterTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo);

	/** Helper to generate the filtered list of types, based on the supported types of the schema */
	UE_API void FilterUnsupportedTypes(TArray<FPinTypeTreeItem>& UnfilteredList);

	/** Helper to generate the filtered list of types, based on the search string matching */
	UE_API bool GetChildrenMatchingSearch(const FText& SearchText, const TArray<FPinTypeTreeItem>& UnfilteredList, TArray<FPinTypeTreeItem>& OutFilteredList, FTopLevenshteinResult<FPinTypeTreeItem>& OutTopLevenshteinResult);

	/** Callback to get the tooltip text for the pin type combo box */
	UE_API FText GetToolTipForComboBoxType() const;

	/** Callback to get the tooltip text for the secondary pin type combo box */
	UE_API FText GetToolTipForComboBoxSecondaryType() const;

	/** Callback to get the tooltip for the array button widget */
	UE_API FText GetToolTipForArrayWidget() const;

	/** Callback to get the tooltip for the container type dropdown widget */
	UE_API FText GetToolTipForContainerWidget() const;

	/** Callback to get the display text for the total pin type item count */
	UE_API FText GetPinTypeItemCountText() const;

	/**
	 * Helper function to create widget for the sub-menu
	 *
	 * @param InItem				Tree item to use for the callback when a menu item is selected
	 * @param InPinType				Pin type for generation of the widget to display for the menu entry
	 * @param InIconBrush			Brush icon to use for the menu entry item
	 * @param InTooltip				The simple tooltip to use for the menu item, an advanced tooltip link will be auto-generated based on the PinCategory
	 */
	UE_API TSharedRef<SWidget> CreateObjectReferenceWidget(FPinTypeTreeItem InItem, FEdGraphPinType& InPinType, const FSlateBrush* InIconBrush, FText InSimpleTooltip) const;

	/** Gets the allowable object types for an tree item, used for building the sub-menu */
	UE_API TSharedRef< SWidget > GetAllowedObjectTypes(FPinTypeTreeItem InItem, bool bForSecondaryType);

	/** Fills the provided array with generated list items based on a pin type's allowed reference types, which could then be used as the item source for a list view */
	UE_API void GenerateAllowedObjectTypesList(TArray<FObjectReferenceListItem>& OutList, FPinTypeTreeItem InItem, bool bForSecondaryType) const;
	
	/**
	 * When a pin type is selected, handle it
	 *
	 * @param InItem				Item selected
	 * @param InPinCategory			This is the PinType's category, must be provided separately as the PinType in the tree item is always Object Types for any object related type.
	 */
	UE_API void OnSelectPinType(FPinTypeTreeItem InItem, FName InPinCategory, bool bForSecondaryType);

	/** Called whenever the custom filter options are changed. */
	UE_API void OnCustomFilterChanged();
};


//////////////////////////////////////////////////////////////////////////
// IPinTypeSelectorFilter

/** An interface for implementing a custom pin type filter for the selector widget. */
class IPinTypeSelectorFilter
{
public:
	virtual ~IPinTypeSelectorFilter() = default;

	/** (Required) - Implement this method to filter the given pin type item and determine whether or not it should be displayed. */
	virtual bool ShouldShowPinTypeTreeItem(FPinTypeTreeItem InItem) const = 0;

	/** (Optional) - Override this method to bind a delegate to call when the filter changes. */
	virtual FDelegateHandle RegisterOnFilterChanged(FSimpleDelegate InOnFilterChanged) { return FDelegateHandle(); }

	/** (Optional) - Override this method to unbind a delegate that was previously bound to a filter change event. */
	virtual void UnregisterOnFilterChanged(FDelegateHandle InDelegateHandle) {}

	/** (Optional) - Override this method to return a widget that allows the user to toggle filter options on/off, etc. */
	virtual TSharedPtr<SWidget> GetFilterOptionsWidget() { return TSharedPtr<SWidget>(); }
};

#undef UE_API
