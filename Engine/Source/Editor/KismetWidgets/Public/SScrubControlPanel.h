// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Framework/SlateDelegates.h"
#include "HAL/Platform.h"
#include "ITransportControl.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "SCurveEditor.h"
#include "SScrubWidget.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API KISMETWIDGETS_API

class SScrubControlPanel : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SScrubControlPanel )
		: _Value( 0 )
		, _NumOfKeys()
		, _SequenceLength()
		, _DisplayDrag(true)
		, _OnValueChanged()
		, _OnBeginSliderMovement()
		, _OnEndSliderMovement()
		, _OnClickedForwardPlay()
		, _OnClickedRecord()
		, _OnClickedBackwardPlay()
		, _OnClickedForwardStep()
		, _OnClickedBackwardStep()
		, _OnClickedForwardEnd()
		, _OnClickedBackwardEnd()
		, _OnClickedToggleLoop()
		, _OnGetLooping()
		, _OnGetPlaybackMode()
		, _OnGetRecording()
		, _ViewInputMin()
		, _ViewInputMax()
		, _OnSetInputViewRange()
		, _OnCropAnimSequence()
		, _OnTickPlayback()
		, _TransportControlWidgetsToCreate()
	{}
		SLATE_ATTRIBUTE( float, Value )
		SLATE_ATTRIBUTE( uint32, NumOfKeys )
		SLATE_ATTRIBUTE( float, SequenceLength )
		SLATE_ARGUMENT( bool, bAllowZoom )
		SLATE_ARGUMENT( bool, bDisplayAnimScrubBarEditing )
		SLATE_ATTRIBUTE(bool, DisplayDrag)
		/** Called when the value is changed by slider or typing */
		SLATE_EVENT( FOnFloatValueChanged, OnValueChanged )
		/** Called right before the slider begins to move */
		SLATE_EVENT( FSimpleDelegate, OnBeginSliderMovement )
		/** Called right after the slider handle is released by the user */
		SLATE_EVENT( FOnFloatValueChanged, OnEndSliderMovement )
		/** Called for slider **/
		//SLATE_EVENT(TAttribute<float>::FGetter, OnValueRead)		
		/** Called when the button is clicked */
		SLATE_EVENT( FOnClicked, OnClickedForwardPlay )
		SLATE_EVENT( FOnClicked, OnClickedRecord )
		SLATE_EVENT( FOnClicked, OnClickedBackwardPlay )
		SLATE_EVENT( FOnClicked, OnClickedForwardStep )
		SLATE_EVENT( FOnClicked, OnClickedBackwardStep )
		SLATE_EVENT( FOnClicked, OnClickedForwardEnd )
		SLATE_EVENT( FOnClicked, OnClickedBackwardEnd )
		SLATE_EVENT( FOnClicked, OnClickedToggleLoop )
		SLATE_EVENT( FOnGetLooping, OnGetLooping )
		SLATE_EVENT( FOnGetPlaybackMode, OnGetPlaybackMode )
		SLATE_EVENT( FOnGetRecording, OnGetRecording )
		/** View Input range **/
		SLATE_ATTRIBUTE( float, ViewInputMin )
		SLATE_ATTRIBUTE( float, ViewInputMax )
		SLATE_EVENT( FOnSetInputViewRange, OnSetInputViewRange )
		/** Called when an anim sequence is cropped before/after a selected frame */
		SLATE_EVENT( FOnCropAnimSequence, OnCropAnimSequence )
		/** Called when an frane is added before/after a selected frame */
		SLATE_EVENT(FOnAddAnimSequence, OnAddAnimSequence)
		/** Called when an frame is appended in the beginning or at the end */
		SLATE_EVENT(FOnAppendAnimSequence, OnAppendAnimSequence)
		/** Called to zero out selected frame's translation from origin */
		SLATE_EVENT( FOnReZeroAnimSequence, OnReZeroAnimSequence )
		SLATE_ATTRIBUTE( bool, IsRealtimeStreamingMode )
		/** Optional, additional values to draw on the timeline **/
		SLATE_ATTRIBUTE( TArray<float>, DraggableBars )
		SLATE_EVENT( FOnScrubBarDrag, OnBarDrag )
		SLATE_EVENT( FOnScrubBarCommit, OnBarCommit )
		/** Called each frame during playback */
		SLATE_EVENT( FOnTickPlayback, OnTickPlayback )
		/**
		 * Array of custom widgets to create in the Transport Control widget. If this array is used the default widget ordering will be
		 * ignored in favor of this set of widgets.
		 */
		SLATE_ARGUMENT(TOptional<TArray<FTransportControlWidget>>, TransportControlWidgetsToCreate);
		SLATE_EVENT(FOnGetRecordButtonVisibility, OnGetRecordVisibility);

	SLATE_END_ARGS()

	/**
	 * Construct the widget
	 * 
	 * @param InArgs   A declaration from which to construct the widget
	 */
	UE_API void Construct( const FArguments& InArgs );

	virtual ~SScrubControlPanel() = default;

protected:
	UE_API EVisibility GetRealtimeControlVisibility(bool bIsControlForRealtimeStreamingMode) const;

protected:
	TSharedPtr<class SScrubWidget>	ScrubWidget;

	TAttribute<bool> IsRealtimeStreamingMode;
};

#undef UE_API
