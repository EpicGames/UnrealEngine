// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Framework/SlateDelegates.h"
#include "HAL/Platform.h"
#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "SCurveEditor.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API KISMETWIDGETS_API

class FPaintArgs;
class FSlateRect;
class FSlateWindowElementList;
class FText;
class FWidgetStyle;
struct FGeometry;
struct FPointerEvent;

/**
 * A Slate SpinBox resembles traditional spin boxes in that it is a widget that provides
 * keyboard-based and mouse-based manipulation of a numberic value.
 * Mouse-based manipulation: drag anywhere on the spinbox to change the value.
 * Keyboard-based manipulation: click on the spinbox to enter text mode.
 */
DECLARE_DELEGATE_TwoParams( FOnCropAnimSequence, bool, float )
DECLARE_DELEGATE_TwoParams( FOnAddAnimSequence, bool, int32 )
DECLARE_DELEGATE_TwoParams( FOnAppendAnimSequence, bool, int32 )
DECLARE_DELEGATE_TwoParams( FOnScrubBarDrag, int32, float)
DECLARE_DELEGATE_TwoParams( FOnScrubBarCommit, int32, float)
DECLARE_DELEGATE_OneParam( FOnReZeroAnimSequence, int32 )

class SScrubWidget : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SScrubWidget)
		: _Value(0)
		, _NumOfKeys(10)
		, _SequenceLength()
		, _bAllowZoom(false)
		, _bDisplayAnimScrubBarEditing(true)
		, _DisplayDrag(true)
		, _OnValueChanged()
		, _OnBeginSliderMovement()
		, _OnEndSliderMovement()
		, _ViewInputMin()
		, _ViewInputMax()
		, _OnSetInputViewRange()
		, _OnCropAnimSequence()
		{}
		
		/** The value to display */
		SLATE_ATTRIBUTE( float, Value )
		SLATE_ATTRIBUTE( uint32, NumOfKeys )
		SLATE_ATTRIBUTE( float, SequenceLength )
		SLATE_ARGUMENT( bool, bAllowZoom )
		SLATE_ARGUMENT( bool, bDisplayAnimScrubBarEditing )
		SLATE_ATTRIBUTE( bool, DisplayDrag )
		/** Called when the value is changed by slider or typing */
		SLATE_EVENT( FOnFloatValueChanged, OnValueChanged )
		/** Called right before the slider begins to move */
		SLATE_EVENT( FSimpleDelegate, OnBeginSliderMovement )
		/** Called right after the slider handle is released by the user */
		SLATE_EVENT( FOnFloatValueChanged, OnEndSliderMovement )
		/** View Input range **/
		SLATE_ATTRIBUTE( float, ViewInputMin )
		SLATE_ATTRIBUTE( float, ViewInputMax )
		SLATE_EVENT( FOnSetInputViewRange, OnSetInputViewRange )
		/** Called when an anim sequence is cropped before/after a selected frame */
		SLATE_EVENT( FOnCropAnimSequence, OnCropAnimSequence )
		/** Called when an frane is added before/after a selected frame */
		SLATE_EVENT( FOnAddAnimSequence, OnAddAnimSequence )
		/** Called when an frame is appended in the beginning or at the end */
		SLATE_EVENT(FOnAppendAnimSequence, OnAppendAnimSequence)
		/** Called to zero out selected frame's translation from origin */
		SLATE_EVENT( FOnReZeroAnimSequence, OnReZeroAnimSequence )
		/** Optional, additional values to draw on the timeline **/
		SLATE_ATTRIBUTE( TArray<float>, DraggableBars )
		SLATE_EVENT( FOnScrubBarDrag, OnBarDrag )
		SLATE_EVENT( FOnScrubBarCommit, OnBarCommit )
	SLATE_END_ARGS()


	/**
	 * Construct the widget
	 * 
	 * @param InArgs   A declaration from which to construct the widget
	 */
	UE_API void Construct( const FArguments& InArgs );

	// SWidget interface
	UE_API virtual FVector2D ComputeDesiredSize(float) const override;
	UE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	UE_API virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	UE_API virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	UE_API virtual FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	UE_API virtual FReply OnMouseWheel( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	UE_API virtual FCursorReply OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const override;

	UE_DEPRECATED(5.6, "Use the helper function accepting FVector2f instead of FVector2D")
	static UE_API int32 GetDivider(float InputMinX, float InputMaxX, FVector2D WidgetSize, float SequenceLength, int32 NumFrames);
	static UE_API int32 GetDivider(float InputMinX, float InputMaxX, FVector2f WidgetSize, float SequenceLength, int32 NumFrames);
protected:

	/**
	 * Call this method when the user's interaction has changed the value
	 *
	 * @param NewValue				Value resulting from the user's interaction
	 * @param bSliderClamp			true to clamp the slider value to the nearest delta value
	 * @param bCommittedFromText	true if the value was committed from the text box and not the slider	
	 */
	UE_API void CommitValue( float NewValue, bool bSliderClamp, bool bCommittedFromText );

private:

	/** Function to create context menu to display anim sequence editing options */
	UE_API void CreateContextMenu( float CurrentFrameTime, const FPointerEvent& MouseEvent );

	/** Function to crop animation sequence before/after selected frame */
	UE_API void OnSequenceCroppedCalled( bool bFromStart, float CurrentFrameTime );

	/** Function to crop animation sequence before/after selected frame */
	UE_API void OnSequenceAddedCalled(bool bBefore, int32 CurrentFrameNumber);

	/** Function to append frames in the beginning or at the end*/
	UE_API void OnSequenceAppendedCalled(const FText & InNewGroupText, ETextCommit::Type CommitInfo, bool bBegin);

	/** Function to ask how many frames to append */
	UE_API void OnShowPopupOfAppendAnimation(bool bBegin);

	/** Function to zero out translation of the selected frame */
	UE_API void OnReZeroCalled(int32 FrameIndex);

	TAttribute<float> ValueAttribute;
	FOnFloatValueChanged OnValueChanged;
	FSimpleDelegate OnBeginSliderMovement;
	FOnFloatValueChanged OnEndSliderMovement;

	TAttribute<float> ViewInputMin;
	TAttribute<float> ViewInputMax;
	FOnSetInputViewRange	OnSetInputViewRange;
	FOnCropAnimSequence		OnCropAnimSequence;
	FOnAddAnimSequence		OnAddAnimSequence;
	FOnAppendAnimSequence	OnAppendAnimSequence;
	FOnReZeroAnimSequence	OnReZeroAnimSequence;

	/** Dragagble bars are generic lines drawn on the scrub widget that can be dragged with the mouse. This is very bare bones and just represents drawing/moving float values */
	TAttribute<TArray<float>>	DraggableBars;
	FOnScrubBarDrag OnBarDrag;
	FOnScrubBarCommit OnBarCommit;

	/** Distance Dragged **/
	float DistanceDragged;
	/** Number of Keys **/
	TAttribute<uint32> NumOfKeys;
	TAttribute<float> SequenceLength;
	/** The number of decimal places to display */
	bool bDragging;
	bool bAllowZoom;
	bool bDisplayAnimScrubBarEditing;
	TAttribute<bool> bDisplayDrag;
	/** If we are currently panning the panel*/
	bool bPanning;
	/** Has the mouse moved during panning - used to determine if we should open the context menu or not */
	bool bMouseMovedDuringPanning;
	/** Bar Dragging*/
	int32 DraggableBarIndex;
	bool DraggingBar;
};

#undef UE_API
