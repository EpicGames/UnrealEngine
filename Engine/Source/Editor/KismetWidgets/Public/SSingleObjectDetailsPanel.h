// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Framework/Commands/UICommandList.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"

#define UE_API KISMETWIDGETS_API

class IDetailsView;
class SWidget;
class UObject;
struct FGeometry;

/////////////////////////////////////////////////////
// SSingleObjectDetailsPanel

class SSingleObjectDetailsPanel : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSingleObjectDetailsPanel) {}
		SLATE_ARGUMENT(TSharedPtr<class FUICommandList>, HostCommandList)
		SLATE_ARGUMENT(TSharedPtr<class FTabManager>, HostTabManager)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, bool bAutomaticallyObserveViaGetObjectToObserve = true, bool bAllowSearch = false);

	// SWidget interface
	UE_API virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	// End of SWidget interface

protected:
	// Should be implemented by derived classes to provide the object being observed
	UE_API virtual UObject* GetObjectToObserve() const;

	// 
	UE_API void SetPropertyWindowContents(TArray<UObject*> Objects);

	//
	UE_API virtual TSharedRef<SWidget> PopulateSlot(TSharedRef<SWidget> PropertyEditorWidget);

	// Property viewing widget
	TSharedPtr<class IDetailsView> PropertyView;

private:
	// Cached object view
	TWeakObjectPtr<UObject> LastObservedObject;

	// If true, GetObjectToObserve will be called every frame to update the object being displayed in the details panel
	// Otherwise, users must call SetPropertyWindowContents manually
	bool bAutoObserveObject;
};


#undef UE_API
